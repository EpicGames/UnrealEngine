// Copyright Epic Games, Inc. All Rights Reserved.

#include "IGDKPackageManifestModule.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformMisc.h"
#include "Serialization/JsonSerializerMacros.h"

#if WITH_GRDK
THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <XPackage.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END
#endif //WITH_GRDK




#if WITH_GRDK
struct FJsonPackageManifestChunkGDK : FJsonSerializable
{
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("id", Data.UnrealChunkID);
		JSON_SERIALIZE("Tag", Data.Tag);
		JSON_SERIALIZE("Languages", Data.Languages);
		JSON_SERIALIZE("IsInitial", Data.bIsInitial);
		JSON_SERIALIZE_ARRAY("files", Files);
	END_JSON_SERIALIZER

	TArray<FString> Files;
	FGDKPackageManifestChunk Data;
};

struct FJsonPackageManifestFeatureGDK : FJsonSerializable
{
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE("id", Data.Id);
		JSON_SERIALIZE("Tags", Data.Tags);
	END_JSON_SERIALIZER

	FGDKPackageManifestFeature Data;
};

struct FJsonPackageManifestGDK: FJsonSerializable
{
	BEGIN_JSON_SERIALIZER
		JSON_SERIALIZE_ARRAY_SERIALIZABLE("chunks", Chunks, FJsonPackageManifestChunkGDK);
		JSON_SERIALIZE_ARRAY_SERIALIZABLE("features", Features, FJsonPackageManifestFeatureGDK);
	END_JSON_SERIALIZER

	TArray<FJsonPackageManifestChunkGDK> Chunks;
	TArray<FJsonPackageManifestFeatureGDK> Features;
};
#endif //WITH_GRDK



class FGDKPackageManifestModule : public IGDKPackageManifestModule
{
public:
	virtual void StartupModule() override
	{
#if WITH_GRDK
		bIsPackagedProcess = XPackageIsPackagedProcess();

		// cache package identifier
		FMemory::Memzero(GamePackageInfo.PackageIdentifier);
		XPackageGetCurrentProcessPackageIdentifier(XPACKAGE_IDENTIFIER_MAX_LENGTH, GamePackageInfo.PackageIdentifier);

		// load the packaging manifest
		FString PackageManifestText = FPlatformMisc::LoadTextFileFromPlatformPackage(TEXT("package.manifest"));
		ParseManifest(PackageManifestText, GamePackageInfo );
#else
		bIsPackagedProcess = false;
#endif
	}


	virtual void ShutdownModule() override
	{
	}

	virtual bool HasManifest() const override
	{
#if WITH_GRDK
		return GamePackageInfo.bManifestLoaded;
#else
		return false;
#endif
	}


	virtual bool IsPakFileInstalled(const FString& InFilename) const override
	{
		return IsPakFileInstalled(nullptr, InFilename);
	}
	
	virtual bool IsPakFileInstalled(XPackageMountHandle MountHandle, const FString& InFilename) const override
	{
#if WITH_GRDK
		// pak files will always be installed in a non-packaged build
		if (!bIsPackagedProcess)
		{
			return true;
		}

		const FPackageManifestInfo& ManifestInfo = GetManifestInfo(MountHandle);

		// lookup the chunk id for this pak file
		const FString CleanFilename = FPaths::GetCleanFilename(InFilename).ToLower();
		const int32* UnrealChunkIDPtr = ManifestInfo.PakToChunkMap.Find(CleanFilename);
		if (UnrealChunkIDPtr == nullptr || (*UnrealChunkIDPtr) < 0)
		{
			return true;
		}

		// build chunk selector
		XPackageChunkSelector ChunkSelector;
		ChunkSelector.type = XPackageChunkSelectorType::Chunk;
		ChunkSelector.chunkId = (*UnrealChunkIDPtr) + ChunkIDOffset;

		// get chunk availability
		XPackageChunkAvailability ChunkAvailability = XPackageChunkAvailability::Unavailable;
		HRESULT hResult = XPackageFindChunkAvailability(ManifestInfo.PackageIdentifier, 1, &ChunkSelector, &ChunkAvailability);
		UE_CLOGF(FAILED(hResult), LogInit, Error, "Failed to query pak chunk %d availability for %ls. 0x%X", *UnrealChunkIDPtr, *CleanFilename, hResult);

		return SUCCEEDED(hResult) && (ChunkAvailability == XPackageChunkAvailability::Ready);
#else
		return true;
#endif
	}

	virtual int32 GetChunkIDFromPakchunkIndex(int32 PakchunkIndex) const override
	{
#if WITH_GRDK
		for (const TPair<FString, int32>& PakToChunkPair : GamePackageInfo.PakToChunkMap)
		{
			int32 FilePakchunkIndex = FPlatformMisc::GetPakchunkIndexFromPakFile(PakToChunkPair.Key);
			if (FilePakchunkIndex == PakchunkIndex)
			{
				return PakToChunkPair.Value;
			}
		}
#endif

		// unknown - fall back to assuming 1:1
		return int32(PakchunkIndex);
	}

	virtual TArray<FString> GetPakFilesInChunk(int32 ChunkID) const override
	{
		TArray<FString> Result;
#if WITH_GRDK
		const TArray<FString>* PakFilesInChunkPtr = GamePackageInfo.ChunkToPakPaths.Find(ChunkID);
		if (PakFilesInChunkPtr)
		{
			Result = (*PakFilesInChunkPtr);
		}
#endif
		return MoveTemp(Result);
	}


	virtual const TArray<FGDKPackageManifestChunk>& GetChunks() const override
	{
		return GamePackageInfo.Chunks;
	}

	virtual const FGDKPackageManifestChunk* GetChunk(int32 UnrealChunkID) const override
	{
		return GamePackageInfo.Chunks.FindByPredicate([UnrealChunkID](const FGDKPackageManifestChunk& Chunk)
		{
			return UnrealChunkID == Chunk.UnrealChunkID;
		});
	}

	virtual const TArray<FGDKPackageManifestFeature>& GetFeatures() const override
	{
		return GamePackageInfo.Features;
	}

	virtual const TArray<FName>& GetLanguages() const override
	{
		return GamePackageInfo.Languages;
	}

	virtual const TArray<FName>& GetTags() const override
	{
		return GamePackageInfo.Tags;
	}

	virtual bool GetUnrealChunkIDsByFeature( const FString& Feature, TArray<int32>& OutUnrealChunkIDs ) const override
	{

#if WITH_GRDK
		const TArray<int32>* ResultPtr = GamePackageInfo.FeatureToUnrealChunkIDMap.Find(Feature);
		if (ResultPtr)
		{
			OutUnrealChunkIDs = *ResultPtr;
			return true;
		}
#endif
		return false;
	}


	// DLC support
	virtual bool LoadManifestFromDLC( XPackageMountHandle MountHandle, const char* PackageIdentifier, const FString& MountPath ) override
	{
#if WITH_GRDK
		FPackageManifestInfo& ManifestInfo = DLCPackageInfo.Add(MountHandle);
		FCStringAnsi::Strncpy(ManifestInfo.PackageIdentifier, PackageIdentifier, XPACKAGE_IDENTIFIER_MAX_LENGTH);

		FString PackageManifestText;
		if (FFileHelper::LoadFileToString(PackageManifestText, &IPlatformFile::GetPlatformPhysical(), *FPaths::Combine( MountPath, TEXT("package.manifest"))))
		{
			ParseManifest(PackageManifestText, ManifestInfo );
		}

		return true;
#else
		return false;
#endif
	}

	virtual void UnloadDLC( XPackageMountHandle MountHandle ) override
	{
#if WITH_GRDK
		DLCPackageInfo.Remove(MountHandle);
#endif
	}


private:

	/**
	 * GDK chunk numbering is off by 2 to account for the registration chunk and binary chunk. To prevent double counting, this is only
	 * applied when sending chunk ids to the system, not for our internal API or data.
	 */
	static const int ChunkIDOffset = 2;

	struct FPackageManifestInfo
	{
#if WITH_GRDK
		char PackageIdentifier[XPACKAGE_IDENTIFIER_MAX_LENGTH];
		TMap<FString, int32> PakToChunkMap;
		TMap<FString, TArray<int32>> FeatureToUnrealChunkIDMap;
		TMap<int32, TArray<FString>> ChunkToPakPaths;

		bool bManifestLoaded;
#endif
		TArray<FGDKPackageManifestChunk> Chunks;
		TArray<FGDKPackageManifestFeature> Features;
		TArray<FName> Languages;
		TArray<FName> Tags;
	};

	FPackageManifestInfo GamePackageInfo;
#if WITH_GRDK
	TMap<XPackageMountHandle, FPackageManifestInfo> DLCPackageInfo;
#endif


	bool bIsPackagedProcess;



#if WITH_GRDK
	bool ParseManifest( const FString& PackageManifestText, FPackageManifestInfo& ManifestInfo )
	{
		TUniquePtr<struct FJsonPackageManifestGDK> Manifest = MakeUnique<FJsonPackageManifestGDK>();
		ManifestInfo.bManifestLoaded = Manifest->FromJson(PackageManifestText);

		if (ManifestInfo.bManifestLoaded)
		{
			TMap<FString,TArray<int32>> ChunkTagToUnrealChunkIDMap;

			for (const FJsonPackageManifestChunkGDK& Chunk : Manifest->Chunks)
			{
				// build pak file lookup table
				for (const FString& Filename : Chunk.Files)
				{
					if (Filename.EndsWith(TEXT(".pak")) || Filename.EndsWith(TEXT(".uondemandtoc")))
					{
						FString CleanFilename = FPaths::GetCleanFilename(Filename).ToLower();
						ManifestInfo.PakToChunkMap.Add(MoveTemp(CleanFilename), Chunk.Data.UnrealChunkID);
						ManifestInfo.ChunkToPakPaths.FindOrAdd(Chunk.Data.UnrealChunkID).Add(Filename);
					}
				}

				// cache chunk data
				ManifestInfo.Chunks.Add(Chunk.Data);

				// cache chunk tag
				TArray<FString> TagArray;
				Chunk.Data.Tag.ParseIntoArray(TagArray, TEXT(";") );
				for (FString Tag : TagArray)
				{
					FString Tag2;
					if (Tag.Split(FString(TEXT("#")), &Tag, &Tag2 ))
					{
						ChunkTagToUnrealChunkIDMap.FindOrAdd(Tag2).AddUnique(Chunk.Data.UnrealChunkID);
						ManifestInfo.Tags.AddUnique(FName(Tag2));
					}
					ChunkTagToUnrealChunkIDMap.FindOrAdd(Tag).AddUnique(Chunk.Data.UnrealChunkID);
					ManifestInfo.Tags.AddUnique(FName(Tag));
				}

				// cache languages
				TArray<FString> LanguageArray;
				Chunk.Data.Languages.ParseIntoArray(LanguageArray, TEXT(";") );
				for (const FString& Language : LanguageArray)
				{
					ManifestInfo.Languages.AddUnique(FName(Language));
				}
			}

			for (const FJsonPackageManifestFeatureGDK& Feature : Manifest->Features)
			{
				// cache feature data
				ManifestInfo.Features.Add(Feature.Data);

				// cache feature tags
				TArray<FString> TagArray;
				Feature.Data.Tags.ParseIntoArray(TagArray, TEXT(";") );
				for (const FString& Tag : TagArray)
				{
					if (const TArray<int32>* UnrealChunkIDsPtr = ChunkTagToUnrealChunkIDMap.Find(Tag))
					{
						TArray<int32>& FeatureChunkIDs = ManifestInfo.FeatureToUnrealChunkIDMap.FindOrAdd(Feature.Data.Id);
						for (int32 UnrealChunkID : *UnrealChunkIDsPtr)
						{
							FeatureChunkIDs.AddUnique(UnrealChunkID);
						}
					}
				}
			}
		}

		return ManifestInfo.bManifestLoaded;
	}


	const FPackageManifestInfo& GetManifestInfo( XPackageMountHandle MountHandle ) const
	{
		if (MountHandle == nullptr)
		{
			return GamePackageInfo;
		}
		else
		{
			const FPackageManifestInfo* ManifestInfo = DLCPackageInfo.Find(MountHandle);
			checkf(ManifestInfo, TEXT("invalid package mount handle"));

			return *ManifestInfo;
		}
	}
#endif //WITH_GRDK

};




IMPLEMENT_MODULE(FGDKPackageManifestModule, GDKPackageManifest);

