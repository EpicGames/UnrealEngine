// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Modules/ModuleManager.h"

struct FGDKPackageManifestChunk
{
	int32 UnrealChunkID = 0;
	FString Tag; // raw tag string with ; and # separators
	FString Languages;
	bool bIsInitial;
};

struct FGDKPackageManifestFeature
{
	FString Id;
	FString Tags; // raw tag string with ; separators
};

typedef struct XPackageMountInstance* XPackageMountHandle;

class IGDKPackageManifestModule : public IModuleInterface, public IPlatformChunkInstallManifest
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IGDKPackageManifestModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IGDKPackageManifestModule>("GDKPackageManifest");
	}

	// determine whether the given pak file is installed or just a download placeholder with no file system backing
	virtual bool IsPakFileInstalled(const FString& InFilename) const = 0;

	// determine if the package manifest was loaded successfully
	virtual bool HasManifest() const override = 0;

	// determine the Unreal chunk ID that the given pak chunk index is stored in
	virtual int32 GetChunkIDFromPakchunkIndex(int32 PakchunkIndex) const override = 0;

	// get the pak files that are stored in the given Unreal chunk
	virtual TArray<FString> GetPakFilesInChunk(int32 ChunkID) const override= 0;


	// get all chunk data
	virtual const TArray<FGDKPackageManifestChunk>& GetChunks() const = 0;

	// get data for a specific unreal chunk
	virtual const FGDKPackageManifestChunk* GetChunk(int32 UnrealChunkID) const = 0;

	// get all feature data
	virtual const TArray<FGDKPackageManifestFeature>& GetFeatures() const = 0;

	// get all languages
	virtual const TArray<FName>& GetLanguages() const = 0;

	// get all tags
	virtual const TArray<FName>& GetTags() const = 0;


	// get chunks that are referred to by the given feature
	virtual bool GetUnrealChunkIDsByFeature( const FString& Feature, TArray<int32>& OutUnrealChunkIDs ) const = 0;

	// DLC support
	virtual bool LoadManifestFromDLC( XPackageMountHandle MountHandle, const char* PackageIdentifier, const FString& MountPath ) = 0;
	virtual void UnloadDLC( XPackageMountHandle MountHandle ) = 0;
	virtual bool IsPakFileInstalled(XPackageMountHandle MountHandle, const FString& InFilename) const = 0;

};
