// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class GDKRuntime : ModuleRules
{
	public GDKRuntime(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateDependencyModuleNames.Add("Core");
		PrivateDependencyModuleNames.Add("GRDK");
	}
}
