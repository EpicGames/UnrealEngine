// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKHandleTracker.h"

#if WITH_GDK_HANDLE_TRACKER

#include "CoreMinimal.h"
#include "HAL/PlatformStackWalk.h"
#include "HAL/PlatformMisc.h"
#include "HAL/IConsoleManager.h"
#include "Misc/ScopeLock.h"
#include "Misc/Crc.h"
#include "Containers/Map.h"

THIRD_PARTY_INCLUDES_START
	#include "Microsoft/AllowMicrosoftPlatformTypes.h"
	#include <atomic>
	#include <XSystem.h>
	#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END


static FAutoConsoleCommand CmdGDKTrackStart(
	TEXT("GDK.XSystemHandle.TrackStart"),
	TEXT("start tracking GDK handles"),
	FConsoleCommandDelegate::CreateStatic( &FGDKHandleTracker::Start )
);

static FAutoConsoleCommand CmdGDKTrackStop(
	TEXT("GDK.XSystemHandle.TrackStop"),
	TEXT("stop tracking GDK handles & dump any opened handles (not necessary a leak)"),
	FConsoleCommandDelegate::CreateStatic( &FGDKHandleTracker::Stop )
);




namespace GDKHandleTrackerInternal
{
	static std::atomic<int32> ActiveCounter = {0};

	// TODO: should these be data-driven? should we have a bit array or something?
	static bool bLimitType = false;
	static XSystemHandleType LimitType = XSystemHandleType::User;

	// callstack tracking data
	struct FGDKTrackedCallstack
	{
		TArray<uint64> Backtrace;
		uint32 RefCount;
		XSystemHandleType HandleType;
	};

	FCriticalSection TrackLock;
	TMap<uint32,FGDKTrackedCallstack> TrackedCallstacks; // hash to callstack
	TMap<XSystemHandle,uint32> TrackedHandles;           // handle to hash

	// high watermark
	int32 MaxTrackedHandles = 0;
	int32 MaxTrackedCallstacks = 0;


	const TCHAR* GetHandleTypeName( XSystemHandleType HandleType )
	{
		switch(HandleType)
		{
			case XSystemHandleType::AppCaptureScreenshotStream: return TEXT("AppCaptureScreenshotStream");
			case XSystemHandleType::DisplayTimeoutDeferral    : return TEXT("DisplayTimeoutDeferral");
			case XSystemHandleType::GameSaveContainer         : return TEXT("GameSaveContainer");
			case XSystemHandleType::GameSaveProvider          : return TEXT("GameSaveProvider");
			case XSystemHandleType::GameSaveUpdate            : return TEXT("GameSaveUpdate");
			case XSystemHandleType::PackageInstallationMonitor: return TEXT("PackageInstallationMonitor");
			case XSystemHandleType::PackageMount              : return TEXT("PackageMount");
			case XSystemHandleType::SpeechSynthesizer         : return TEXT("SpeechSynthesizer");
			case XSystemHandleType::SpeechSynthesizerStream   : return TEXT("SpeechSynthesizerStream");
			case XSystemHandleType::StoreContext              : return TEXT("StoreContext");
			case XSystemHandleType::StoreLicense              : return TEXT("StoreLicense");
			case XSystemHandleType::StoreProductQuery         : return TEXT("StoreProductQuery");
			case XSystemHandleType::TaskQueue                 : return TEXT("TaskQueue");
			case XSystemHandleType::User                      : return TEXT("User");
			case XSystemHandleType::UserSignOutDeferral       : return TEXT("UserSignOutDeferral");
			default: return TEXT("(unknown)");
		}
	}

	uint32 TrackCallstack( XSystemHandleType HandleType )
	{
		// capture callstack
		static const uint32 MaxBacktraceDepth = 32;
		uint64 Backtrace[MaxBacktraceDepth];
		uint32 BacktraceDepth = FPlatformStackWalk::CaptureStackBackTrace( Backtrace, MaxBacktraceDepth );

		// skip the actual capture & tracking items
		static const uint32 BacktraceEntriesToIgnore = 4; // xgameruntime handle tracker -> GDKSystemHandleCallback() -> TrackCallstack() -> FPlatformStackWalk::CaptureStackBackTrace()
		uint64* BacktraceStart = Backtrace + BacktraceEntriesToIgnore;
		BacktraceDepth -= BacktraceEntriesToIgnore;

		// hash the callstack and see if we have it already
		uint32 CallstackHash = FCrc::MemCrc32(BacktraceStart, sizeof(uint64) * BacktraceDepth, 0);
		FGDKTrackedCallstack* Existing = TrackedCallstacks.Find(CallstackHash);
		if (Existing == nullptr)
		{
			Existing = &TrackedCallstacks.Add( CallstackHash );
			Existing->Backtrace.Append(BacktraceStart, BacktraceDepth);
			Existing->RefCount = 0;
			Existing->HandleType = HandleType;
		}
		else
		{
			// shouldn't ever get two handles from the same callstack (if we do, they're being created from within one GDK handle creation function which is unlikely. Would need to roll the handle type into the hash in this case)
			check(Existing->HandleType == HandleType);
		}

		// add reference & return
		Existing->RefCount++;
		return CallstackHash;
	}


	void FreeCallstack(uint32 CallstackHash)
	{
		// lookup the callstack and clear it if this is the last reference
		FGDKTrackedCallstack* Existing = TrackedCallstacks.Find(CallstackHash);
		if (ensure(Existing != nullptr))
		{
			Existing->RefCount--;
			if (Existing->RefCount <= 0)
			{
				TrackedCallstacks.Remove(CallstackHash);
			}
		}
	}


	static void CALLBACK GDKSystemHandleCallback( XSystemHandle Handle, XSystemHandleType HandleType, XSystemHandleCallbackReason Reason, void* Context )
	{
		// filter by type
		if (bLimitType && LimitType != HandleType)
		{
			return;
		}

		if (Reason == XSystemHandleCallbackReason::Created)
		{
			// track this handle's callstack & update high watermark
			FScopeLock Lock(&TrackLock);
			TrackedHandles.Add(Handle, TrackCallstack(HandleType) );

			MaxTrackedCallstacks = FMath::Max( MaxTrackedCallstacks, TrackedCallstacks.Num() );
			MaxTrackedHandles = FMath::Max( MaxTrackedHandles, TrackedHandles.Num() );
		}
		else if (Reason == XSystemHandleCallbackReason::Destroyed)
		{
			// release this handle's callstack
			FScopeLock Lock(&TrackLock);
			const uint32* CallstackPtr = TrackedHandles.Find(Handle);
			if (CallstackPtr != nullptr) // NB. we may not find it if tracking has already been dumped & restarted
			{
				FreeCallstack(*CallstackPtr);
				TrackedHandles.Remove(Handle);
			}
		}
	}

	// note: uses LowLevelOutputDebugString not UE_LOG because it's typically called after engine shutdown & UE_LOG is no longer available
	static void DumpHandles()
	{
		FScopeLock Lock(&TrackLock);

		// dump callstacks & handle counters
		if (TrackedHandles.Num() > 0 )
		{
			FPlatformMisc::LowLevelOutputDebugString( TEXT("\n\n"));

			for ( const TPair<uint32,FGDKTrackedCallstack>& TrackedCallstack : TrackedCallstacks )
			{
				FPlatformMisc::LowLevelOutputDebugStringf( TEXT("%d active %s GDK handles detected from callstack:\n"), TrackedCallstack.Value.RefCount, GetHandleTypeName(TrackedCallstack.Value.HandleType) );

				// dump callstack
				for (int32 Depth = 0; Depth < TrackedCallstack.Value.Backtrace.Num(); Depth++)
				{
					ANSICHAR Buffer[1024];
					Buffer[0] = '\0';
					FPlatformStackWalk::ProgramCounterToHumanReadableString(Depth, TrackedCallstack.Value.Backtrace[Depth], Buffer, sizeof(Buffer) );
					FPlatformMisc::LowLevelOutputDebugStringf( TEXT("\t%s\n"), ANSI_TO_TCHAR(Buffer));
				}
				FPlatformMisc::LowLevelOutputDebugString( TEXT("\n\n"));
			}
		}

		// dump final report & reset trackers
		FPlatformMisc::LowLevelOutputDebugStringf( TEXT("Total %d handles, %d unique callstacks. (Maximum was %d handles, %d unique callstacks)\n"), TrackedHandles.Num(), TrackedCallstacks.Num(), MaxTrackedHandles, MaxTrackedCallstacks);
		MaxTrackedHandles = 0;
		MaxTrackedCallstacks = 0;
		TrackedCallstacks.Reset();
		TrackedHandles.Reset();
	}
}

void FGDKHandleTracker::Start()
{
	if ((++GDKHandleTrackerInternal::ActiveCounter) == 1)
	{
		HRESULT hResult = XSystemHandleTrack( GDKHandleTrackerInternal::GDKSystemHandleCallback, nullptr);
		if (FAILED(hResult))
		{
			FPlatformMisc::LowLevelOutputDebugStringf( TEXT("Failed to start GDK handle tracking : 0x%X\n"), hResult);
		}
	}
}

void FGDKHandleTracker::Stop()
{
	if ((--GDKHandleTrackerInternal::ActiveCounter) == 0)
	{
		CA_SUPPRESS(6387) // first parameter is tagged _In_ but the documentation says nullptr used to disable handle tracking
		XSystemHandleTrack(nullptr, nullptr);
		GDKHandleTrackerInternal::DumpHandles();
	}
	else
	{
		FPlatformMisc::LowLevelOutputDebugString( TEXT("Other GDK trackers are still active - ignoring stop\n"));
	}
}



#endif //WITH_GDK_HANDLE_TRACKER
