// Copyright Epic Games, Inc. All Rights Reserved.


#include "GDKRuntimeModule.h"
#if WITH_GRDK
#include "GDKHandleTracker.h"
#include "HAL/Platform.h"
#include "HAL/PlatformProcess.h" 
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "Misc/App.h"
#include "Misc/FileHelper.h"
#include "Containers/AnsiString.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <grdk.h>
#include <XGameRuntimeInit.h>
#include <XError.h>
#include <XGameErr.h>
#include <appmodel.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"


#if WITH_GRDK_DEV_RUNTIME_INIT
static HRESULT InitXGameRuntimeForDevelopment(const TCHAR* ConfigIniSection);
#endif


static int32 GDKEnvironmentInitRefCount = 0;
static bool bGDKEnvironmentInitialized = false;

static bool bGDKIsTrackingHandles = false;


bool SetupGDKEnvironment(const TCHAR* ConfigIniSection)
{
	// initialize on first call only
	if ((++GDKEnvironmentInitRefCount) > 1)
	{
		return bGDKEnvironmentInitialized;
	}

	// GDK runtime is not needed for commandlets
	if (IsRunningCommandlet())
	{
		return false;
	}

	// initialize the gaming runtime
#if WITH_GRDK_DEV_RUNTIME_INIT
	HRESULT hResult = InitXGameRuntimeForDevelopment(ConfigIniSection);
	if (FAILED(hResult))
	{
		hResult = XGameRuntimeInitialize();
	}
#else
	HRESULT hResult = XGameRuntimeInitialize();
#endif

	// Workaround race condition with Clang where xgameruntime.dll might not be present by this point
#if defined(__clang__) && !PLATFORM_WINDOWS
	int32 RetryCount = 0;
	while (hResult == E_GAMERUNTIME_DLL_NOT_FOUND && RetryCount < 100)
	{
		// Sleep 200ms and try again
		FPlatformProcess::Sleep(0.200f);
		UE_LOGF(LogGDK, Warning, "Error: Game Runtime dll wasn't found! Retrying");
		hResult = XGameRuntimeInitialize();
		RetryCount++;
	}
#endif // __clang__ && !PLATFORM_WINDOWS

	// check for specific errors
#if PLATFORM_WINDOWS
	UE_CLOGF(hResult == E_GAMERUNTIME_DLL_NOT_FOUND, LogGDK, Error, "Error: Game Runtime is not installed");
	UE_CLOGF(hResult == E_GAMERUNTIME_VERSION_MISMATCH, LogGDK, Error, "Game Runtime version mismatch");
	UE_CLOGF(hResult==0x80073CFC, LogGDK, Error, "GamingServicesRuntime may need reinstalling" );
	UE_CLOGF(hResult==0x87e5001f/*E_GAME_MISSING_GAME_CONFIG*/, LogGDK, Error, "Missing MicrosoftGame.config");
#else
	UE_CLOGF(hResult == E_GAMERUNTIME_DLL_NOT_FOUND, LogGDK, Fatal, "Error: Game Runtime is not installed");
	UE_CLOGF(hResult == E_GAMERUNTIME_VERSION_MISMATCH, LogGDK, Fatal, "Game Runtime version mismatch");
#endif
	UE_CLOGF(FAILED(hResult), LogGDK, Error, "XGameRuntimeInitialize failed: 0x%X", hResult);

	// early out of the runtime failed to initialize
	bGDKEnvironmentInitialized = SUCCEEDED(hResult);
	if (!bGDKEnvironmentInitialized)
	{
		return false;
	}

	// check for gdk lifetime command line features
#if WITH_GDK_HANDLE_TRACKER
	if (FParse::Param(FCommandLine::Get(),TEXT("gdkhandletrack") ) )
	{
		bGDKIsTrackingHandles = true;
		FGDKHandleTracker::Start();
	}
#endif

	return true;
}

void TeardownGDKEnvironment()
{
	if ((--GDKEnvironmentInitRefCount) != 0)
	{
		return;
	}

	if (!bGDKEnvironmentInitialized)
	{
		return;
	}

#if WITH_GDK_HANDLE_TRACKER
	if (bGDKIsTrackingHandles)
	{
		FGDKHandleTracker::Stop();
		bGDKIsTrackingHandles = false;
	}
#endif

	XErrorSetOptions(XErrorOptions::OutputDebugStringOnError, XErrorOptions::OutputDebugStringOnError); // #jira UE-141916 October GDK bug: do not break on error because there are active objects on WinGDK shutdown (even in a new Direct3D 12 Desktop Game project.) 
	XGameRuntimeUninitialize();

	bGDKEnvironmentInitialized = false;
}


#if WITH_GRDK_DEV_RUNTIME_INIT
static FAnsiString FindMSGameConfigFile()
{
	static const TCHAR* MSGameConfig = TEXT("MicrosoftGame.config");
	
	// check project root
	FString MSGameConfigFile = FPaths::Combine( FPlatformProcess::BaseDir(), TEXT("../../.."), MSGameConfig);
	if (FPaths::FileExists(MSGameConfigFile))
	{
		return *MSGameConfigFile;
	}

	// check binaries directory
	MSGameConfigFile = FPaths::Combine( FPlatformProcess::BaseDir(), MSGameConfig);
	if (FPaths::FileExists(MSGameConfigFile))
	{
		return *MSGameConfigFile;
	}

	// check the saved folder
#if !WITH_EDITOR
	// this is primarily to support F5 in Visual Studio without specifying a -basedir ... in this case the game is loading cooked content from the Saved/Cooked directory or via Zen streaming
	if (FPaths::IsProjectFilePathSet())
	{
		MSGameConfigFile = FPaths::Combine( FPaths::ProjectSavedDir(), TEXT("Win64"), TEXT("Manifest"), TEXT("MicrosoftGame.config") );
		if (FPaths::FileExists(MSGameConfigFile))
		{
			return *MSGameConfigFile;
		}
	}
#endif

	return FAnsiString();
}
#endif // WITH_GRDK_DEV_RUNTIME_INIT






#if WITH_GRDK_DEV_RUNTIME_INIT
static FAnsiString MakeInlineMSGameConfig(const TCHAR* ConfigSectionName )
{
	// must have config section (code issue)
	if (ConfigSectionName == nullptr)
	{
		UE_LOGF(LogGDK, Warning, "Config section name has not been specified.");
		return FAnsiString();
	}

	// Newer GDKs require these two properties to be specified before the default user can be added. 
	// While many GDK runtime functions still function without a user, most of the useful functionality is user-focused so just disable GDK runtime if we can't add a user.
	FString TitleId, MSAAppId;
	if (!GConfig->GetString(ConfigSectionName, TEXT("TitleId"),  TitleId,  GEngineIni) || TitleId.IsEmpty() ||
		!GConfig->GetString(ConfigSectionName, TEXT("MSAAppId"), MSAAppId, GEngineIni) || MSAAppId.IsEmpty())
	{
		UE_LOGF(LogGDK, Warning, "TitleId and MSAppId must be specified in MSGamingRuntime plugin settings before GDK runtime can be used.");
		return FAnsiString();
	}

	// Read package identity
	FString PackageName = FApp::GetProjectName();
	GConfig->GetString(ConfigSectionName, TEXT("PackageName"), PackageName, GEngineIni);
	if (PackageName.IsEmpty())
	{
		PackageName = TEXT("DefaultUEProject");
	}

	FString PublisherName;
	GConfig->GetString(ConfigSectionName, TEXT("PublisherName"), PublisherName, GEngineIni);
	if (PublisherName.IsEmpty())
	{
		PublisherName = TEXT("CN=NoPublisher");
	}


	// Write a smallest-possible placeholder MicrosoftGame.config file required to initialize the GDK runtime.
	return FAnsiString::Printf("<?xml version=\"1.0\" encoding=\"utf-8\"?><Game configVersion=\"1\"><Identity Name=\"%S\" Publisher=\"%S\"/><TitleId>%S</TitleId><MSAAppId>%S</MSAAppId></Game>", *PackageName, *PublisherName, *TitleId, *MSAAppId );
}
#endif // WITH_GRDK_DEV_RUNTIME_INIT



#if WITH_GRDK_DEV_RUNTIME_INIT
static bool IsPackagedProcess()
{
	// editor will never be packaged 
#if WITH_EDITOR
	return false;
#else

	// get the kernel
	HMODULE hModule = ::GetModuleHandleW(TEXT("kernel32.dll"));
	if (hModule == nullptr)
	{
		return false;
	}

	// look up the GetCurrentPackageId function
PRAGMA_DISABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS	// unsafe conversion from 'type of expression' to 'type required'
	typedef LONG(WINAPI *GetCurrentPackageIdProc)(UINT32*, BYTE*);
	GetCurrentPackageIdProc fnGetCurrentPackageId = (GetCurrentPackageIdProc)::GetProcAddress(hModule, "GetCurrentPackageId");
PRAGMA_ENABLE_CAST_FUNCTION_TYPE_MISMATCH_WARNINGS

	if (fnGetCurrentPackageId == nullptr)
	{
		return false;
	}

	UINT32 BufferLength = 0;
	return fnGetCurrentPackageId(&BufferLength, nullptr) != APPMODEL_ERROR_NO_PACKAGE;
#endif  //WITH_EDITOR
}
#endif // WITH_GRDK_DEV_RUNTIME_INIT



#if WITH_GRDK_DEV_RUNTIME_INIT
static HRESULT InitXGameRuntimeForDevelopment(const TCHAR* ConfigIniSection)
{
	HRESULT hResult = E_FAIL;

	// early out if this is a packaged build because debugging assistance is not required
	if (IsPackagedProcess())
	{
		UE_LOGF(LogGDK, Display, "Debugging packaged build");
		return E_FAIL;
	}

	// check for a MicrosoftGame.config in known locations
	FAnsiString MSGameConfigFile = FindMSGameConfigFile();
	if (!MSGameConfigFile.IsEmpty())
	{
		XGameRuntimeOptions Options;
		Options.gameConfigSource = XGameRuntimeGameConfigSource::File;
		Options.gameConfig = *MSGameConfigFile;
		hResult = XGameRuntimeInitializeWithOptions(&Options);
	}

	// make a placeholder minimal MicrosoftGame.config - just enough to initialize
	if (FAILED(hResult) && hResult != E_GAMERUNTIME_OPTIONS_NOT_SUPPORTED)
	{
		FAnsiString MSGameConfig = MakeInlineMSGameConfig(ConfigIniSection);
		if (!MSGameConfig.IsEmpty())
		{
			XGameRuntimeOptions Options;
			Options.gameConfigSource = XGameRuntimeGameConfigSource::Inline;
			Options.gameConfig = *MSGameConfig;
			hResult = XGameRuntimeInitializeWithOptions(&Options);
#if WITH_EDITOR
			UE_CLOGF(SUCCEEDED(hResult), LogGDK, Display, "Using minimal XGameRuntime config for PIE - most settings ignored" );
#else
UE_CLOGF(SUCCEEDED(hResult), LogGDK, Warning, "Using temporary XGameRuntime config - most settings ignored" );
#endif
			checkf(hResult != E_GAMERUNTIME_GAMECONFIG_BAD_FORMAT, TEXT("autogenerated MSGameConfig is malformed - check the code/GDK release notes"));
		}
	}


	// this error code indicates that there is a packaged version of the build already installed
	// in this case we must currently create a temporary minimal MicrosoftGame.config file alongside the .exe
	// to work around a GDK bug
	if (hResult == E_GAMERUNTIME_OPTIONS_NOT_SUPPORTED)
	{
		// give some details that might be helpful
		UE_LOGF(LogGDK, Display, "NOTE: There is a packaged version of this game installed as well but it is not the one being debugged."); 
		UE_LOGF(LogGDK, Display, "To debug that in visual studio, use Debug -> Other Debug Targets -> Debug Installed App Package.");
		UE_LOGF(LogGDK, Display, "To debug this executable against the installed package's content, add the path to the installed package's binary folder to the command line.  e.g. -basedir=C:\\XboxGames\\[YourApp]\\Content\\[YourGame]\\Binaries\\Win64\\  (use wdapp.exe list /d to discover the install path)"); 

		// delete any stale file
		FString TempMSGameConfigFile = FPaths::Combine(FPaths::GetPath(FPlatformProcess::ExecutablePath()), TEXT("MicrosoftGame.config"));
		if (IPlatformFile::GetPlatformPhysical().FileExists(*TempMSGameConfigFile))
		{
			IPlatformFile::GetPlatformPhysical().DeleteFile(*TempMSGameConfigFile);
		}

		// generate a temporary MicrosoftGame.config alongside the exe file
		FAnsiString MSGameConfig = MakeInlineMSGameConfig(ConfigIniSection);
		if (FFileHelper::SaveStringToFile(ANSI_TO_TCHAR(*MSGameConfig), *TempMSGameConfigFile, FFileHelper::EEncodingOptions::ForceAnsi))
		{
			hResult = XGameRuntimeInitialize();
			IPlatformFile::GetPlatformPhysical().DeleteFile(*TempMSGameConfigFile);
		}
	}


	// cannot find gaming config - have to initialize without one. 
	// this likely means that we can't sign in a user or connect to xsapi etc. because we won't have a package identity
	if (FAILED(hResult))
	{
		hResult = XGameRuntimeInitialize();
		UE_CLOGF(SUCCEEDED(hResult), LogGDK, Warning, "Likely using XGameRuntime without a MicrosoftGame.config" );
	}

	return hResult;
}
#endif // WITH_GRDK_DEV_RUNTIME_INIT


#endif //WITH_GRDK
