// Copyright Epic Games, Inc. All Rights Reserved.


#include "GDKRuntimeModule.h"
#if WITH_GRDK
#include "GDKHandle.h"
#include "GDKTaskQueueHelpers.h"
#include "GDKThreadCheck.h"
#include "GDKHandleTracker.h"
#include "GDKUserManager.h"
#include "GenericPlatform/GenericPlatformChunkInstall.h"
#include "Modules/ModuleManager.h"
#include "CoreGlobals.h"
#include "Misc/MessageDialog.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Misc/Optional.h"
#include "Misc/Paths.h"
#include "Misc/CommandLine.h"
#include "Math/NumericLimits.h"
#include "Stats/Stats.h"
#include "HAL/FileManager.h"
#include "HAL/ThreadSafeBool.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformAtomics.h"
#include "Templates/UniquePtr.h"
#include "Containers/Ticker.h"
#include "Microsoft/COMPointer.h"
#include "Async/Async.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
#include <XGameRuntimeInit.h>
#include <XGameRuntimeFeature.h>
#include <XGame.h>
#include <XSystem.h>
#include <XError.h>
#include <XGameErr.h>
#include <XUser.h>
#include <XNetworking.h>
#include <XThread.h>
#include <XDisplay.h>
#include <XGameInvite.h>
#include <ipifcons.h>
#include <grdk.h>

#if _GRDK_EDITION >= 230300
#include <XGameProtocol.h>
#endif
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"


// XGameProtocol support added in 230300
#if !defined(GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL)
	#define GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL (_GRDK_EDITION >= 230300)
#endif

// GDK 230300 documentation says ms-xbl-<titleid> is now reserved for game invites. 
// However, newer OS versions also use it for "Play Here Instead" passing "ms-xbl-<titleid>:///" on the command line
#if !defined(GDK_PROTOCOL_SUPPORT_MS_XBL)
	#define GDK_PROTOCOL_SUPPORT_MS_XBL 1
#endif

static TAutoConsoleVariable<FString> CVarGDKDefaultGamertagComponent(
	TEXT("GDK.DefaultGamertagComponent"),
	"Classic",
	TEXT("Choose one from [Classic, Modern, ModernSuffix, UniqueModern]. Refer to GDK document for more info.")
);

#if GDK_PROTOCOL_SUPPORT_MS_XBL
FString GDKTitleIdUriPrefix = TEXT("ms-xbl://");
static FAutoConsoleVariableRef CVarGDKTitleIdUriPrefix(
	TEXT("GDK.TitleIdUriPrefix"),
	GDKTitleIdUriPrefix,
	TEXT("Prefix for protocol activation Uri when triggered via ms-xbl-<titleid>://")
);
#endif //GDK_PROTOCOL_SUPPORT_MS_XBL




DEFINE_LOG_CATEGORY(LogGDK);


class FGDKRuntimeModule : public IGDKRuntimeModule
{
public:
	virtual void StartupModule() override
	{
		Init();
	}
	virtual void ShutdownModule() override
	{
		Teardown();
	}

	virtual bool IsAvailable() const override;

	virtual EAppReturnType::Type MessageBoxExt(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption) const override;

	virtual ENetworkConnectionType GetNetworkConnectionType() const override;
	virtual bool HasActiveWiFiConnection() const override;

	virtual uint32 GetTitleId() const override;
	virtual FString GetXboxSandboxId() const override;
	virtual FString GetPrimaryServiceConfigId(bool bAllowPlaceholder) const override;
	virtual FString GetGamertag(const FGDKUserHandle& UserHandle) const override;
	virtual FString GetGamertag(const FGDKUserHandle& UserHandle, XUserGamertagComponent GamertagComponent) const override;
	virtual XUserGamertagComponent GetDefaultGamertagComponent() const override;

	virtual const TCHAR* GetConfigSectionName() const override;
	virtual bool IsUsingSimplifiedUserModel() const override;
	virtual bool IsNetworkInitialized() const override;

	virtual void DisableScreenTimeout() override;
	virtual void EnableScreenTimeout() override;

	virtual void Internal_HandlePlatformTextFieldVisible( bool bIsVisible ) override;
	virtual bool Internal_IsPlatformTextFieldVisible() const override;

	virtual HRESULT AsyncGDKTask( TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction, XTaskQueueHandle TaskQueue ) const override;
	virtual HRESULT AsyncGDKTask( class FGDKAsyncTaskMonitor& OutMonitor, TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction, XTaskQueueHandle TaskQueue ) const override;
	virtual XTaskQueueHandle GetGenericTaskQueue() const override;
	virtual XTaskQueueHandle GetBackgroundTaskQueue() const override;


	virtual FGDKUserHandle GetUserHandleByPlatformId(FPlatformUserId PlatformUserId) const override;
	virtual FGDKUserHandle GetUserHandleByXUserId(uint64 XUserId) const override;
	virtual FPlatformUserId GetPlatformIdByUserHandle(const FGDKUserHandle& UserHandle) override;
	virtual TOptional<uint64> GetXUserIdByPlatformId(FPlatformUserId PlatformUserId) const override;
	virtual TArray<FGDKUserHandle> GetAllUserHandles() const override;
	virtual bool WasUserRecentlySignedOut(XUserLocalId UserLocalId) const override;
	virtual void PickUserAsync(XUserAddOptions Options, FGDKPickUserCompleteDelegate Delegate) const override;
	virtual FPlatformUserId GetUnpairedPlatformId() const override;

	virtual void SetRuntimeErrorProcessDelegate(HRESULT hResult, FGDKRuntimeErrorProcessDelegate Delegate) override;
	virtual void ClearRuntimeErrorProcessDelegate(HRESULT hResult) override;

	virtual FDelegateHandle RegisterGameInviteReceivedDelegate( FGDKOnGameInviteReceivedDelegate Delegate ) override;
	virtual void UnregisterGameInviteReceivedDelegate( const FDelegateHandle& DelegateHandle ) override;


	FGDKUserManager& GetUserManager() const { return *UserManagerInstance.Get(); }
private:

	static void OnNetworkConnectivityChangeCallback( void*, const struct XNetworkingConnectivityHint* ConnectivityHint );
#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL || GDK_PROTOCOL_SUPPORT_MS_XBL
	static void OnProtocolActivationCallback( void*, const char* ProtocolUri );	
#endif
	void SetNetworkConnectionType( const struct XNetworkingConnectivityHint* ConnectivityHint );
	void SetDefaultGamertagComponent(IConsoleVariable* CVar);

	void SetProtocolActivationUri(const FString& NewUriString);
	bool TranslateUri( const FString& SourceUri, FString& OutDstUri ) const;
#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL || GDK_PROTOCOL_SUPPORT_MS_XBL
	bool ParseAndRemoveProtocolUri( FString& InOutCommandLine );
#endif
	static void OnGameInviteReceivedCallback( void*, const char* ProtocolUri );	

	void Init();
	void Teardown();
	void CacheConfiguration();
	bool bGDKEnvironmentInitialized = false;
	bool bHasMSGamingModules = false;

	TUniquePtr<FGDKUserManager> UserManagerInstance;

	ENetworkConnectionType CachedNetworkConnectionType = ENetworkConnectionType::Unknown;
	TOptional<XTaskQueueRegistrationToken> NetworkConnectivityEventToken;
	bool bCachedNetworkInitialized = false;
	TOptional<FDelegateHandle> NetworkInitializedSuspendHandle;
	TOptional<FDelegateHandle> NetworkInitializedResumeHandle;

	FString ProtocolActivationUri;
	FTSTicker::FDelegateHandle ProtocolActivationTickerHandle;
#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL
	TOptional<XTaskQueueRegistrationToken> ProtocolActivationRegistrationToken;
	TMap<FString,FString> UriActivationProtocols;
#endif
	TOptional<XTaskQueueRegistrationToken> GameInviteRegistrationToken;
	FGDKOnGameInviteReceived OnGameInviteReceivedDelegate;
	FString LastGameInviteRecivedUri;


	XDisplayTimeoutDeferralHandle TimeoutDeferralHandle = nullptr;
	XUserGamertagComponent DefaultGamertagComponent = XUserGamertagComponent::Classic;

	int32 PlatformTextFieldVisibleCounter = 0;

	struct FGDKEnvironment
	{
		uint32 TitleId = 0;
		FString XboxLiveSandboxId;
		FString PrimaryServiceConfigId;
		bool bHasValidPrimaryServiceConfigId = false;
	} GDKEnvironment;

	TMap<HRESULT, FGDKRuntimeErrorProcessDelegate> RuntimeErrorProcessDelegates;
	FCriticalSection RuntimeErrorProcessDelegatesLock;

#if WITH_EDITOR
	virtual void Internal_InitForPIE() override;
	virtual void Internal_TeardownForPIE() override;
	virtual TMulticastDelegate<void()>& GetOnInitForPIE() override { return OnInitForPIEDelegate; }
	virtual TMulticastDelegate<void()>& GetOnTeardownForPIE() override { return OnTeardownForPIEDelegate; }

	TMulticastDelegate<void()> OnInitForPIEDelegate;
	TMulticastDelegate<void()> OnTeardownForPIEDelegate;
#endif

};

IMPLEMENT_MODULE(FGDKRuntimeModule, GDKRuntime);






static FAutoConsoleCommand CmdGDKDebugDump(
	TEXT("GDK.DumpUsers"),
	TEXT("Logs out all current user states"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		FGDKRuntimeModule& RuntimeInstance = ((FGDKRuntimeModule&)IGDKRuntimeModule::Get());
		RuntimeInstance.GetUserManager().DebugDump();
	})
);


static FAutoConsoleCommand CmdGDKDebugAddUser(
	TEXT("GDK.AddDebugUser"),
	TEXT("Adds a new XUser - debug only!"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		IGDKRuntimeModule::Get().PickUserAsync(XUserAddOptions::AllowGuests, nullptr);
	})
);







void FGDKRuntimeModule::Init()
{
#if PLATFORM_WINDOWS
	// cache whether any of the MSGaming modules are part of the project. This determines where the GDK target settings are read from
	bHasMSGamingModules = FModuleManager::Get().ModuleExists(TEXT("MSGamingRuntime")) ||
							FModuleManager::Get().ModuleExists(TEXT("MSGamingSupport")) ||
							FModuleManager::Get().ModuleExists(TEXT("MSGameStore"));
#else
	bHasMSGamingModules = false;
#endif

	// initialize GDK runtime
	bGDKEnvironmentInitialized = SetupGDKEnvironment(GetConfigSectionName());
	if (!bGDKEnvironmentInitialized)
	{
		return;
	}


	// init task queues
	FGDKAsyncTaskQueue::PlatformInit();


#if WITH_GDK_THREAD_CHECK
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XThread) && IsGDKTimeSensitiveThreadCheckEnabled())
	{
		// mark the game thread as time sensitive
		check(IsInGameThread());
		XThreadSetTimeSensitive(true);

		// mark RHI and RenderThreads as time sensitive once they are up and running
		AsyncTask( ENamedThreads::RHIThread,[]()
		{ 
			XThreadSetTimeSensitive(true);
		});

		AsyncTask( ENamedThreads::ActualRenderingThread,[]()
		{ 
			XThreadSetTimeSensitive(true);
		});
	}
#endif

	SetDefaultGamertagComponent(CVarGDKDefaultGamertagComponent.AsVariable());
	CVarGDKDefaultGamertagComponent.AsVariable()->OnChangedDelegate().AddRaw(this, &FGDKRuntimeModule::SetDefaultGamertagComponent);

	// Error callback message hook
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XError))
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XErrorSetOptions & XErrorSetCallback are not safe to call on time-sensitive threads
		XErrorSetOptions(XErrorOptions::DebugBreakOnError | XErrorOptions::OutputDebugStringOnError, XErrorOptions::OutputDebugStringOnError);
		XErrorSetCallback([](HRESULT hResult, const char* Msg, void* Context)
		{
			if (FGDKRuntimeModule* This = reinterpret_cast<FGDKRuntimeModule*>(Context))
			{
				FScopeLock ScopeLock(&This->RuntimeErrorProcessDelegatesLock);
				FGDKRuntimeErrorProcessDelegate* DelegatePtr = This->RuntimeErrorProcessDelegates.Find(hResult);
				if (DelegatePtr != nullptr)
				{
					FGDKRuntimeErrorProcessDelegate Delegate = *DelegatePtr;
					if (Delegate.Execute(Msg))
					{
						return true;
					}
				}
			}

			if (hResult == E_OUTOFMEMORY)
			{
				// Cannot allocate memory during error handling if we are out of memory.  We will recurse and eventually exceed the stack.
				char OutOfMemoryMessageBuffer[2 * 1024];
				TCString<char>::Snprintf(OutOfMemoryMessageBuffer, sizeof(OutOfMemoryMessageBuffer), "XGameError: 0x%X - %s.\n", hResult, Msg ? Msg : "(no message)");
				OutputDebugStringA(OutOfMemoryMessageBuffer);
			}
			else
			{
				UE_LOGF( LogGDK, Error, "XGameError: 0x%X - %ls.", hResult, Msg ? UTF8_TO_TCHAR(Msg) : TEXT("(no message)"));
			}
			return true;
		}, this);
	}



	// Network connection change hooks
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
	{
		XTaskQueueRegistrationToken Token;
		if (SUCCEEDED(XNetworkingRegisterConnectivityHintChanged( FGDKAsyncTaskQueue::GetGenericQueue(), this, &FGDKRuntimeModule::OnNetworkConnectivityChangeCallback, &Token)))
		{
			NetworkConnectivityEventToken = Token;
		}

		// Get current network initialization status (very likely not initialized)
		XNetworkingConnectivityHint ConnectivityHint = {};
		const HRESULT GetConnectivityHintResult = XNetworkingGetConnectivityHint(&ConnectivityHint);
		if (SUCCEEDED(GetConnectivityHintResult))
		{
			SetNetworkConnectionType(&ConnectivityHint);
		}
		else
		{
			UE_LOGF(LogGDK, Error, "Failed to determine new network initialization status due to error. Error=[%0.8X]", GetConnectivityHintResult);
			FPlatformMisc::SetNetworkConnectionStatus(ENetworkConnectionStatus::Disabled);
		}

		// Bind to application suspend delegate so we can update the network initialization status when we suspend
		NetworkInitializedSuspendHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddLambda([this]()
		{
			SCOPED_ENTER_BACKGROUND_EVENT(STAT_Networking_ApplicationWillEnterBackgroundLambda);

			// Pretend the network is gone as we're about to suspend
			if (bCachedNetworkInitialized != false)
			{
				bCachedNetworkInitialized = false;
				FCoreDelegates::ApplicationNetworkInitializationChanged.Broadcast(bCachedNetworkInitialized);
			}
		});

		// Bind to application resume delegate so we can update the network initialization status when we resume
		NetworkInitializedResumeHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddLambda([this]()
		{
			if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking))
			{
				XNetworkingConnectivityHint ConnectivityHint = {};
				const HRESULT GetConnectivityHintLambdaResult = XNetworkingGetConnectivityHint(&ConnectivityHint);
				if (SUCCEEDED(GetConnectivityHintLambdaResult))
				{
					SetNetworkConnectionType(&ConnectivityHint);
				}
				else
				{
					UE_LOGF(LogGDK, Error, "Failed to determine new network initialization status due to error. Error=[%0.8X]", GetConnectivityHintLambdaResult);
				}
			}
			else
			{
				SetNetworkConnectionType(nullptr);
			}
		});
	}
	else
	{
		SetNetworkConnectionType(nullptr);
	}

PRAGMA_DISABLE_DEPRECATION_WARNINGS // XGameProtocolRegisterForActivation & XGameInviteRegisterForEvent are deprecated in 260400
	// protocol activation
#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XGame))
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XGameProtocolRegisterForActivation is not safe to call on time-sensitive threads
		XTaskQueueRegistrationToken Token;
		if (SUCCEEDED(XGameProtocolRegisterForActivation( FGDKAsyncTaskQueue::GetGenericQueue(), this, FGDKRuntimeModule::OnProtocolActivationCallback, &Token )))
		{
			ProtocolActivationRegistrationToken = Token;
		}
	}
#endif

	// game invite hooking (not used for actual game invites, just accessing ms-xbl-xxx)
	if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XGameInvite))
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XGameInviteRegisterForEvent is not safe to call on time-sensitive thread
		XTaskQueueRegistrationToken Token;
		if (SUCCEEDED(XGameInviteRegisterForEvent( FGDKAsyncTaskQueue::GetGenericQueue(), this, FGDKRuntimeModule::OnGameInviteReceivedCallback, &Token )))
		{
			GameInviteRegistrationToken = Token;
		}
	}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


	// create the GDK user manager
	UserManagerInstance.Reset(new FGDKUserManager);

	// attempt to add the default user (i.e the user that launched the title on Xbox, or the user signed in to the xbox game bar on Windows)
	// NB. not done in the GDK user manager constructor because there's a chance it will trigger OSS delegates that call back to FGDKUserManager
	UserManagerInstance->AddDefaultUser();


	// Environment Information.
	if (GConfig != nullptr && GConfig->IsReadyForUse())
	{
		CacheConfiguration();
	}
	else
	{
		FCoreDelegates::TSConfigReadyForUse().AddRaw(this, &FGDKRuntimeModule::CacheConfiguration);
	}
}

void FGDKRuntimeModule::Teardown()
{
	if (bGDKEnvironmentInitialized)
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS // XGameProtocolUnregisterForActivation & XGameInviteUnregisterForEvent are deprecated in 260400

#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL
		if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XGame) && ProtocolActivationRegistrationToken.IsSet())
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) XGameProtocolUnregisterForActivation is not safe to call on time-sensitive threads
			XGameProtocolUnregisterForActivation(ProtocolActivationRegistrationToken.GetValue(), true);
			ProtocolActivationRegistrationToken.Reset();
		}
#endif

		if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XGameInvite) && GameInviteRegistrationToken.IsSet())
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) XGameInviteUnregisterForEvent is not safe to call on time-sensitive threads
			XGameInviteUnregisterForEvent(GameInviteRegistrationToken.GetValue(), true);
			GameInviteRegistrationToken.Reset();
		}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

		if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XNetworking) && NetworkConnectivityEventToken.IsSet() )
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) XNetworkingUnregisterConnectivityHintChanged is not safe to call on time-sensitive threads
			XNetworkingUnregisterConnectivityHintChanged(NetworkConnectivityEventToken.GetValue(), true);
			NetworkConnectivityEventToken.Reset();
		}

		if (XGameRuntimeIsFeatureAvailable(XGameRuntimeFeature::XError))
		{
			GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) XErrorSetCallback is not safe to call on time-sensitive threads
			XErrorSetCallback(nullptr, nullptr);
		}

		// shut down task queues
		FGDKAsyncTaskQueue::PlatformTeardown();

		// destroy the GDK user manager
		UserManagerInstance.Reset();
	}

	// clean up GDK
	TeardownGDKEnvironment();
	bGDKEnvironmentInitialized = false;

	// remove ticker for uri protocol handling
	if (ProtocolActivationTickerHandle.IsValid())
	{
		FTSTicker::GetCoreTicker().RemoveTicker(ProtocolActivationTickerHandle);
		ProtocolActivationTickerHandle.Reset();
	}
}

bool FGDKRuntimeModule::IsAvailable() const
{
	return bGDKEnvironmentInitialized;
}

ENetworkConnectionType FGDKRuntimeModule::GetNetworkConnectionType() const
{
	return CachedNetworkConnectionType;
}

bool FGDKRuntimeModule::HasActiveWiFiConnection() const
{
	return CachedNetworkConnectionType == ENetworkConnectionType::WiFi;
}

void FGDKRuntimeModule::OnNetworkConnectivityChangeCallback( void* Context, const XNetworkingConnectivityHint* ConnectivityHint )
{
	((FGDKRuntimeModule*)Context)->SetNetworkConnectionType(ConnectivityHint);
}

void FGDKRuntimeModule::SetNetworkConnectionType( const XNetworkingConnectivityHint* ConnectivityHint )
{
	ENetworkConnectionType OldConnectionType = CachedNetworkConnectionType;

	bool bOldNetworkInitialization = bCachedNetworkInitialized;
	bCachedNetworkInitialized = ConnectivityHint && ConnectivityHint->networkInitialized;

	if (bCachedNetworkInitialized)
	{
		if (ConnectivityHint->ianaInterfaceType == IF_TYPE_ETHERNET_CSMACD)
		{
			UE_CLOGF( (OldConnectionType != CachedNetworkConnectionType), LogGDK, Log, "Network connectivity level: Ethernet" );
			CachedNetworkConnectionType = ENetworkConnectionType::Ethernet;
		}
		else if (ConnectivityHint->ianaInterfaceType == IF_TYPE_IEEE80211)
		{
			UE_CLOGF( (OldConnectionType != CachedNetworkConnectionType), LogGDK, Log, "Network connectivity level: Wifi" );
			CachedNetworkConnectionType = ENetworkConnectionType::WiFi;
		}
	}
	else if (CachedNetworkConnectionType != ENetworkConnectionType::Unknown)
	{
		UE_CLOGF( (OldConnectionType != CachedNetworkConnectionType), LogGDK, Log, "Network connectivity level: Unknown" );
		CachedNetworkConnectionType = ENetworkConnectionType::Unknown;
	}

	if (ConnectivityHint == nullptr)
	{
		// If we can't tell, assume the network is available
		bCachedNetworkInitialized = true;
	}

	const ENetworkConnectionStatus NewNetworkConnectionStatus = (ConnectivityHint && ConnectivityHint->connectivityLevel == XNetworkingConnectivityLevelHint::InternetAccess) ? ENetworkConnectionStatus::Connected : ENetworkConnectionStatus::Disabled;
	FPlatformMisc::SetNetworkConnectionStatus(NewNetworkConnectionStatus);

	if (bOldNetworkInitialization != bCachedNetworkInitialized)
	{
		FCoreDelegates::ApplicationNetworkInitializationChanged.Broadcast(bCachedNetworkInitialized);
	}
}


void FGDKRuntimeModule::CacheConfiguration()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XGameGetXboxTitleId & XGameGetXboxLiveSandboxId are not safe to call on a time-critical thread

	// title id
	HRESULT hResult;
	hResult = XGameGetXboxTitleId(&GDKEnvironment.TitleId);
	UE_CLOGF(FAILED(hResult), LogGDK, Display, "failed to get TitleId: %x", hResult);

	// xbl sandbox id
	char SandboxIdBuf[XSystemXboxLiveSandboxIdMaxBytes] = { 0 };
	hResult = XSystemGetXboxLiveSandboxId(XSystemXboxLiveSandboxIdMaxBytes, SandboxIdBuf, nullptr);
	UE_CLOGF(FAILED(hResult), LogGDK, Display, "failed to get SandboxId: %x", hResult);
	if (SUCCEEDED(hResult))
	{
		GDKEnvironment.XboxLiveSandboxId = UTF8_TO_TCHAR(SandboxIdBuf);
	}

	// xbl service config id
	check(GConfig && GConfig->IsReadyForUse());
	FString ConfiguredSCID;
	if (GConfig->GetString(TEXT("OnlineSubsystemGDK"), TEXT("PrimaryServiceConfigId"), ConfiguredSCID, GEngineIni) && !ConfiguredSCID.IsEmpty())
	{
		GDKEnvironment.PrimaryServiceConfigId = ConfiguredSCID;
		GDKEnvironment.bHasValidPrimaryServiceConfigId = true;
	}
	else if (GConfig->GetString(GetConfigSectionName(), TEXT("PrimaryServiceConfigId"), ConfiguredSCID, GEngineIni) && !ConfiguredSCID.IsEmpty())
	{
		GDKEnvironment.PrimaryServiceConfigId = ConfiguredSCID;
		GDKEnvironment.bHasValidPrimaryServiceConfigId = true;
	}
	else
	{
		GDKEnvironment.PrimaryServiceConfigId = FString::Printf(TEXT("00000000-0000-0000-0000-0000%08X"), GetTitleId());
		GDKEnvironment.bHasValidPrimaryServiceConfigId = false;
	}

	// show results
	UE_LOGF(LogGDK, Display, "TitleId: %x", GDKEnvironment.TitleId);
	UE_LOGF(LogGDK, Display, "SandboxId: %ls", *GDKEnvironment.XboxLiveSandboxId );
	UE_LOGF(LogGDK, Display, "Primary Service Configuration Id: %ls", *GDKEnvironment.PrimaryServiceConfigId );
	UE_LOGF(LogGDK, Display, "GDK Edition: %d", _GRDK_EDITION);


	// cache activation protocols
#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL
	TArray<FString> RawActivationProtocols;
	GConfig->GetArray(GetConfigSectionName(), TEXT("UriActivationProtocols"), RawActivationProtocols, GEngineIni);
	for (FString& RawActivationProtocol : RawActivationProtocols)
	{
		RawActivationProtocol.TrimStartAndEndInline();
		RawActivationProtocol.TrimCharInline(TEXT('('), nullptr);
		RawActivationProtocol.TrimCharInline(TEXT(')'), nullptr);

		FString RegisteredProtocol;
		FString InGameUriPrefix;
		if (FParse::Value(*RawActivationProtocol, TEXT("RegisteredProtocol="), RegisteredProtocol) && FParse::Value(*RawActivationProtocol, TEXT("InGameUriPrefix="), InGameUriPrefix ) )
		{
			RegisteredProtocol.Split(TEXT(":"), &RegisteredProtocol, nullptr); // ignore any suffix in case someone has added a :// etc.
			RegisteredProtocol.ToLowerInline();                                // protocol must be lower case

			UriActivationProtocols.Add(RegisteredProtocol, InGameUriPrefix);
		}
	}

	// check command line now
#if GDK_PROTOCOL_SUPPORT_MS_XBL
	// cvars may not be initialized yet, so read the value directly
	GConfig->GetString(TEXT("ConsoleVariables"), TEXT("GDK.TitleIdUriPrefix"), GDKTitleIdUriPrefix, GEngineIni);

	FString CmdLine(FCommandLine::Get());
	if (ParseAndRemoveProtocolUri(CmdLine))
	{
		FCommandLine::Set(*CmdLine);
	}
#endif //GDK_PROTOCOL_SUPPORT_MS_XBL
#endif //GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL
}


uint32 FGDKRuntimeModule::GetTitleId() const
{
	return GDKEnvironment.TitleId;
}

FString FGDKRuntimeModule::GetXboxSandboxId() const
{
	return GDKEnvironment.XboxLiveSandboxId;
}

FString FGDKRuntimeModule::GetPrimaryServiceConfigId(bool bAllowPlaceholder) const
{
	if (GDKEnvironment.bHasValidPrimaryServiceConfigId || bAllowPlaceholder)
	{
		return GDKEnvironment.PrimaryServiceConfigId;
	}
	else
	{
		return FString();
	}
}

void LexFromString(XUserGamertagComponent& Value, const TCHAR* ComponentName)
{
	if (FCString::Stricmp(ComponentName, TEXT("Classic")) == 0)
	{
		Value = XUserGamertagComponent::Classic;
	}
	else if (FCString::Stricmp(ComponentName, TEXT("Modern")) == 0)
	{
		Value = XUserGamertagComponent::Modern;
	}
	else if (FCString::Stricmp(ComponentName, TEXT("ModernSuffix")) == 0)
	{
		Value = XUserGamertagComponent::ModernSuffix;
	}
	else if (FCString::Stricmp(ComponentName, TEXT("UniqueModern")) == 0)
	{
		Value = XUserGamertagComponent::UniqueModern;
	}
	else
	{
		UE_LOGF(LogGDK, Warning, "[LexFromString] ComponentName FString [%ls] did not match any XUserGamertagComponent. Returning XUserGamertagComponent::Classic as default.", ComponentName);

		Value = XUserGamertagComponent::Classic;
	}
}

size_t GetMaxBytesForGamertagComponent(XUserGamertagComponent GamertagComponent)
{
	switch (GamertagComponent)
	{
	case XUserGamertagComponent::Classic:
		return XUserGamertagComponentClassicMaxBytes;
	case XUserGamertagComponent::Modern:
		return XUserGamertagComponentModernMaxBytes;
	case XUserGamertagComponent::ModernSuffix:
		return XUserGamertagComponentModernSuffixMaxBytes;
	case XUserGamertagComponent::UniqueModern:
		return XUserGamertagComponentUniqueModernMaxBytes;
	default:
		checkNoEntry();
		return 0Ui64;
	}
}

void FGDKRuntimeModule::SetDefaultGamertagComponent(IConsoleVariable* CVar)
{
	FString UserGamertagComponentStr = CVar->AsVariable()->GetString();
	LexFromString(DefaultGamertagComponent, *UserGamertagComponentStr);
}

FString FGDKRuntimeModule::GetGamertag(const FGDKUserHandle& UserHandle) const
{
	return GetGamertag(UserHandle, DefaultGamertagComponent);
}

FString FGDKRuntimeModule::GetGamertag(const FGDKUserHandle& UserHandle, XUserGamertagComponent GamertagComponent) const
{
	if (!UserHandle.IsValid())
	{
		return TEXT("(invalid user)");
	}

	FString Gamertag;

	size_t GamertagComponentMaxBytes = GetMaxBytesForGamertagComponent(GamertagComponent);
	TUniquePtr<char[]> GamerTagBuffer = MakeUnique<char[]>(GamertagComponentMaxBytes + 1);
	size_t GamerTagBufferUsedSize = 0;
	HRESULT Result = XUserGetGamertag(UserHandle, GamertagComponent, GamertagComponentMaxBytes, GamerTagBuffer.Get(), &GamerTagBufferUsedSize);
	if (SUCCEEDED(Result))
	{
		Gamertag = UTF8_TO_TCHAR(GamerTagBuffer.Get());
	}
	else
	{
		UE_LOGF(LogGDK, Warning, "[FGDKRuntimeModule::GetGamertag] Failed to get user gamertag. Result = (0x%0.8X)", Result);
	}

	return Gamertag;
}

XUserGamertagComponent FGDKRuntimeModule::GetDefaultGamertagComponent() const 
{
	return DefaultGamertagComponent;
}

const TCHAR* FGDKRuntimeModule::GetConfigSectionName() const
{
#if PLATFORM_WINDOWS
	static FString ConfigSectionName = [this]()
	{
		return bHasMSGamingModules ? TEXT("/Script/MSGamingSupport.MSGamingSettings") : FPlatformProperties::GetRuntimeSettingsClassName();
	}();

	return *ConfigSectionName;
#else
	return FPlatformProperties::GetRuntimeSettingsClassName();
#endif
}

bool FGDKRuntimeModule::IsUsingSimplifiedUserModel() const
{
	static bool bIsUsingSimplifiedUserModel = []()
	{
		bool bResult = false;
		GConfig->GetBool(IGDKRuntimeModule::Get().GetConfigSectionName(), TEXT("bUseSimplifiedUserModel"), bResult, GEngineIni);
		return bResult;
	}();

	return bIsUsingSimplifiedUserModel;
}

bool FGDKRuntimeModule::IsNetworkInitialized() const
{
	return bCachedNetworkInitialized;
}

void FGDKRuntimeModule::DisableScreenTimeout()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XDisplayAcquireTimeoutDeferral is not safe to call on a time-sensitive thread

	if (TimeoutDeferralHandle != nullptr)
	{
		XDisplayCloseTimeoutDeferralHandle(TimeoutDeferralHandle);
		TimeoutDeferralHandle = nullptr;
	}

	HRESULT hResult = XDisplayAcquireTimeoutDeferral(&TimeoutDeferralHandle);
	UE_CLOGF(FAILED(hResult), LogGDK, Display, "failed to acquire timeout deferral handle: %x", hResult);
}

void FGDKRuntimeModule::EnableScreenTimeout()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XDisplayCloseTimeoutDeferralHandle is not safe to call on a time-sensitive thread

	XDisplayCloseTimeoutDeferralHandle(TimeoutDeferralHandle);
}


void FGDKRuntimeModule::Internal_HandlePlatformTextFieldVisible( bool bIsVisible )
{
	if (bIsVisible)
	{
		FPlatformAtomics::InterlockedIncrement(&PlatformTextFieldVisibleCounter);
	}
	else
	{
		FPlatformAtomics::InterlockedDecrement(&PlatformTextFieldVisibleCounter);
		check(FPlatformAtomics::AtomicRead(&PlatformTextFieldVisibleCounter) >= 0);
	}
}

bool FGDKRuntimeModule::Internal_IsPlatformTextFieldVisible() const
{
	return (FPlatformAtomics::AtomicRead(&PlatformTextFieldVisibleCounter) > 0);
}



class FGDKMessagBox
{
public:
	FGDKMessagBox( const TCHAR* InText, const TCHAR* InCaption )
		: Text(InText)
		, Caption(InCaption)
		, DefaultButton(XGameUiMessageDialogButton::First)
		, CancelButton(XGameUiMessageDialogButton::Second)
		, bIsFinished(false)
		, Result(EAppReturnType::Cancel)
		, NumButtons(0)
	{
		for( int i = 0; i < MaxButtons; i++ )
		{
			Buttons[i].Text = nullptr;
		}

		check( int(XGameUiMessageDialogButton::First) == 0 );
		check( int(XGameUiMessageDialogButton::Second) == 1 );
		check( int(XGameUiMessageDialogButton::Third) == 2 );
	}

	void AddButton( const char* InText, EAppReturnType::Type Id, bool bIsDefault = false )
	{
		if( NumButtons < MaxButtons )
		{
			FButton& Button = Buttons[NumButtons];
			Button.Text = InText;
			Button.Id = Id;

			if( Id == EAppReturnType::Cancel )
			{
				CancelButton = (XGameUiMessageDialogButton)NumButtons;
			}
			if( bIsDefault )
			{
				DefaultButton = (XGameUiMessageDialogButton)NumButtons;
			}

			NumButtons++;
		}
	}

	EAppReturnType::Type Show()
	{ 
		FGDKLocalTaskBlock Block;
		if( SUCCEEDED(XGameUiShowMessageDialogAsync( Block, TCHAR_TO_UTF8(Caption), TCHAR_TO_UTF8(Text), Buttons[0].Text, Buttons[1].Text, Buttons[2].Text, DefaultButton, CancelButton )) )
		{
			Block.BlockUntilComplete();

			XGameUiMessageDialogButton Button;
			if (SUCCEEDED(XGameUiShowMessageDialogResult( Block, &Button)))
			{
				int ButtonIndex = (int)Button;
				Result = Buttons[ButtonIndex].Id;
			}

			bIsFinished = true; 
		}

		return Result;
	}

protected:


	struct FButton
	{
		const char* Text;
		EAppReturnType::Type Id;
	};

	static const int MaxButtons = 3;

	const TCHAR* Text;
	const TCHAR* Caption;

	XGameUiMessageDialogButton DefaultButton;
	XGameUiMessageDialogButton CancelButton;

	FThreadSafeBool bIsFinished;
	EAppReturnType::Type Result;

	int NumButtons;
	FButton Buttons[MaxButtons];
};


EAppReturnType::Type FGDKRuntimeModule::MessageBoxExt(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption) const
{
	FGDKMessagBox GDKMessageBox( Text, Caption );

	//add the appropriate buttons
	switch( MsgType )
	{
		case EAppMsgType::Ok:
			GDKMessageBox.AddButton( "OK",         EAppReturnType::Ok );
			break;

		case EAppMsgType::YesNo:
			GDKMessageBox.AddButton( "Yes",        EAppReturnType::Yes );
			GDKMessageBox.AddButton( "No",         EAppReturnType::No );
			break;

		case EAppMsgType::OkCancel:
			GDKMessageBox.AddButton( "OK",         EAppReturnType::Ok );
			GDKMessageBox.AddButton( "Cancel",     EAppReturnType::Cancel );
			break;

		case EAppMsgType::YesNoCancel:
			GDKMessageBox.AddButton( "Yes",        EAppReturnType::Yes );
			GDKMessageBox.AddButton( "No",         EAppReturnType::No );
			GDKMessageBox.AddButton( "Cancel",     EAppReturnType::Cancel );
			break;

		case EAppMsgType::CancelRetryContinue:
			GDKMessageBox.AddButton( "Cancel",     EAppReturnType::Cancel );
			GDKMessageBox.AddButton( "Retry",      EAppReturnType::Retry, true );
			GDKMessageBox.AddButton( "Continue",   EAppReturnType::Continue );
			break;

		case EAppMsgType::YesNoYesAll:
			GDKMessageBox.AddButton( "Yes",        EAppReturnType::Yes );
			GDKMessageBox.AddButton( "No",         EAppReturnType::No );
			GDKMessageBox.AddButton( "Yes To All", EAppReturnType::YesAll );
			break;

		default:
			UE_LOGF(LogGDK, Warning, "MsgBox: type not supported\n" );
			return FGenericPlatformMisc::MessageBoxExt(MsgType, Text, Caption);
	}

	EAppReturnType::Type Result = GDKMessageBox.Show();
	return Result;
}




void FGDKRuntimeModule::SetProtocolActivationUri(const FString& NewUriString)
{
	ProtocolActivationUri = NewUriString;

	// start or stop the ticker as appropriate
	if (!ProtocolActivationUri.IsEmpty())
	{
		UE_LOGF(LogGDK, Log, "Got protocol activation Uri : %ls", *NewUriString );

		if (!ProtocolActivationTickerHandle.IsValid())
		{
			ProtocolActivationTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda(
				[this](float)
				{
					if (FCoreDelegates::OnActivatedByProtocol.IsBound())
					{
						FCoreDelegates::OnActivatedByProtocol.Broadcast(ProtocolActivationUri, PLATFORMUSERID_NONE);

						ProtocolActivationUri.Reset();
						ProtocolActivationTickerHandle.Reset();
						return false; // remove from ticker
					}
				
					return true; // try again next tick
				}
			));
		}
	}
	else if (ProtocolActivationTickerHandle.IsValid())
	{
		FTSTicker::GetCoreTicker().RemoveTicker(ProtocolActivationTickerHandle);
		ProtocolActivationTickerHandle.Reset();
	}
}

bool FGDKRuntimeModule::TranslateUri( const FString& SourceUri, FString& OutDstUri ) const
{
	// note: only basic Uri forms are supported at the moment    scheme:payload and scheme://payload ... no authority support
	FString UriScheme;
	FString UriPayload;
	if (SourceUri.Split(FString(TEXT("://")), &UriScheme, &UriPayload) || SourceUri.Split(FString(TEXT(":")), &UriScheme, &UriPayload))
	{
		UriScheme.ToLowerInline();

#if GDK_PROTOCOL_SUPPORT_MS_XBL
		if (UriScheme.StartsWith(TEXT("ms-xbl-")))
		{
			OutDstUri = GDKTitleIdUriPrefix + UriPayload;
			return true;
		}
#endif

#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL
		const FString* UriPrefixPtr = UriActivationProtocols.Find(UriScheme);
		if (UriPrefixPtr != nullptr)
		{
			OutDstUri = (*UriPrefixPtr) + UriPayload;
			return true;
		}
#endif
	}

	return false;
}

#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL || GDK_PROTOCOL_SUPPORT_MS_XBL
bool FGDKRuntimeModule::ParseAndRemoveProtocolUri( FString& InOutCommandLine )
{
	bool bResult = false;

	const TCHAR* RawCmdLine = *InOutCommandLine;
	FString SourceUri;
	if (FParse::Token(RawCmdLine, SourceUri, false))
	{
		FString TranslatedUri;
		if (TranslateUri(SourceUri, TranslatedUri))
		{
			// update the command line now the Uri token has been parsed out
			FString UpdatedCommandLine(RawCmdLine);
			InOutCommandLine = UpdatedCommandLine;

			FPlatformMisc::LowLevelOutputDebugStringf(TEXT("Launched with URI: %s\n"), *SourceUri );
			SetProtocolActivationUri(TranslatedUri);

			bResult = true;
		}
	}

	return bResult;
}
#endif

#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL || GDK_PROTOCOL_SUPPORT_MS_XBL
void FGDKRuntimeModule::OnProtocolActivationCallback( void* Context, const char* ProtocolUri )
{
	FGDKRuntimeModule* RuntimeModule = (FGDKRuntimeModule*)Context;

	FString SourceUri(UTF8_TO_TCHAR(ProtocolUri));
	FString TranslatedUri;
	if (RuntimeModule->TranslateUri(SourceUri, TranslatedUri))
	{
		RuntimeModule->SetProtocolActivationUri(TranslatedUri);
	}
}
#endif

void FGDKRuntimeModule::OnGameInviteReceivedCallback( void* Context, const char* InviteUri )
{
	FGDKRuntimeModule* RuntimeModule = (FGDKRuntimeModule*)Context;

	// record the newest invite Uri and broadcast it to all listeners
	RuntimeModule->LastGameInviteRecivedUri = UTF8_TO_TCHAR(InviteUri);
	ExecuteOnGameThread( UE_SOURCE_LOCATION, [RuntimeModule, Uri = RuntimeModule->LastGameInviteRecivedUri]()
	{
		RuntimeModule->OnGameInviteReceivedDelegate.Broadcast(Uri);
	});

	// forward this to the normal protocol activation handler too, for deep linking & 'play here instead' support
#if GDK_PROTOCOL_SUPPORT_XGAMEPROTOCOL || GDK_PROTOCOL_SUPPORT_MS_XBL
	OnProtocolActivationCallback(Context, InviteUri);
#endif
}



HRESULT FGDKRuntimeModule::AsyncGDKTask( TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction, XTaskQueueHandle TaskQueue ) const
{
	return ::AsyncGDKTask(InitFunction, ResultFunction, TaskQueue);
}

HRESULT FGDKRuntimeModule::AsyncGDKTask( class FGDKAsyncTaskMonitor& OutMonitor, TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction, XTaskQueueHandle TaskQueue ) const
{
	return ::AsyncGDKTask(OutMonitor, InitFunction, ResultFunction, TaskQueue);
}

XTaskQueueHandle FGDKRuntimeModule::GetGenericTaskQueue() const
{
	return FGDKAsyncTaskQueue::GetGenericQueue();
}

XTaskQueueHandle FGDKRuntimeModule::GetBackgroundTaskQueue() const
{
	return FGDKAsyncTaskQueue::GetBackgroundTaskQueue();
}




FGDKUserHandle FGDKRuntimeModule::GetUserHandleByPlatformId(FPlatformUserId PlatformUserId) const
{
	return GetUserManager().GetUserHandleByPlatformId(PlatformUserId);
}

FGDKUserHandle FGDKRuntimeModule::GetUserHandleByXUserId(uint64 XUserId) const
{
	return GetUserManager().GetUserHandleByXUserId(XUserId);
}

FPlatformUserId FGDKRuntimeModule::GetPlatformIdByUserHandle(const FGDKUserHandle& UserHandle)
{
	return GetUserManager().GetPlatformIdByUserHandle(UserHandle);
}

TOptional<uint64> FGDKRuntimeModule::GetXUserIdByPlatformId(FPlatformUserId PlatformUserId) const
{
	return GetUserManager().GetXUserIdByPlatformId(PlatformUserId);
}

TArray<FGDKUserHandle> FGDKRuntimeModule::GetAllUserHandles() const
{
	return GetUserManager().GetAllUserHandles();
}

bool FGDKRuntimeModule::WasUserRecentlySignedOut(XUserLocalId UserLocalId) const
{
	return GetUserManager().WasUserRecentlySignedOut(UserLocalId);
}

void FGDKRuntimeModule::PickUserAsync(XUserAddOptions Options, FGDKPickUserCompleteDelegate Delegate) const
{
	return GetUserManager().PickUserAsync(Options, Delegate);
}

FPlatformUserId FGDKRuntimeModule::GetUnpairedPlatformId() const
{
	return FPlatformUserId::CreateFromInternalId( GetUserManager().FindEmptySeat() );
}

void FGDKRuntimeModule::SetRuntimeErrorProcessDelegate(HRESULT hResult, FGDKRuntimeErrorProcessDelegate Delegate)
{
	FScopeLock ScopeLock(&RuntimeErrorProcessDelegatesLock);
	RuntimeErrorProcessDelegates.Emplace(hResult, Delegate);
}

void FGDKRuntimeModule::ClearRuntimeErrorProcessDelegate(HRESULT hResult)
{
	FScopeLock ScopeLock(&RuntimeErrorProcessDelegatesLock);
	RuntimeErrorProcessDelegates.Remove(hResult);
}

FDelegateHandle FGDKRuntimeModule::RegisterGameInviteReceivedDelegate( FGDKOnGameInviteReceivedDelegate Delegate )
{
	// if we've received an existing Uri already, send it to this new callback now
	if (!LastGameInviteRecivedUri.IsEmpty())
	{
		ExecuteOnGameThread( UE_SOURCE_LOCATION, [Delegate, Uri = LastGameInviteRecivedUri]()
		{
			Delegate.ExecuteIfBound(Uri);
		});
	}

	return OnGameInviteReceivedDelegate.Add(Delegate);
}

void FGDKRuntimeModule::UnregisterGameInviteReceivedDelegate( const FDelegateHandle& DelegateHandle )
{
	OnGameInviteReceivedDelegate.Remove(DelegateHandle);
}


#if WITH_EDITOR
void FGDKRuntimeModule::Internal_InitForPIE()
{
	Init();
	OnInitForPIEDelegate.Broadcast();
}
void FGDKRuntimeModule::Internal_TeardownForPIE()
{
	OnTeardownForPIEDelegate.Broadcast();
	Teardown();
}
#endif



FString LexToString( const FGDKUserHandle& UserHandle )
{
	if (!UserHandle.IsValid())
	{
		return TEXT("<null user>");
	}

	FString Result = IGDKRuntimeModule::Get().GetGamertag(UserHandle);

	if (Result.IsEmpty())
	{
		Result = FString(TEXT("<invalid user>"));
	}

	return Result;
}



FString LexToString( const XUserLocalId& UserId )
{
	FString Result;
	FGDKUserHandle UserHandle;
	if (SUCCEEDED(XUserFindUserByLocalId(UserId, UserHandle.GetInitReference())))
	{
		Result = LexToString(UserHandle);
	}
	else
	{
		Result = FString::Printf( TEXT("<invalid id: %llx>"), UserId.value );
	}

	return Result;
}


const TCHAR* LexToString(XNetworkingConnectivityLevelHint Value)
{
	switch (Value)
	{
	case XNetworkingConnectivityLevelHint::None:						return TEXT("None");
	case XNetworkingConnectivityLevelHint::ConstrainedInternetAccess:	return TEXT("ConstrainedInternetAccess");
	case XNetworkingConnectivityLevelHint::InternetAccess:				return TEXT("InternetAccess");
	case XNetworkingConnectivityLevelHint::LocalAccess:					return TEXT("LocalAccess");
	case XNetworkingConnectivityLevelHint::Unknown:						return TEXT("Unknown");
	}

	checkNoEntry();
	return TEXT("Invalid");
}


#endif //WITH_GRDK
