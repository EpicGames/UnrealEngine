// Copyright Epic Games, Inc. All Rights Reserved.


#include "GDKTaskQueueHelpers.h"
#if WITH_GRDK
#include "GDKThreadCheck.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformMisc.h"
#include "Containers/Ticker.h"
#include "Tasks/Pipe.h"
#include "Templates/UniquePtr.h"

FGDKAsyncBlock::FGDKAsyncBlock(void* InUserData, FGDKAsyncBlockDelegate InDelegate, XTaskQueueHandle TaskQueue)
	: UserData(InUserData)
	, Delegate(MoveTemp(InDelegate))
{
	// init to zero
	AsyncBlock = XAsyncBlock{ 0 };

	// default to our generic queue
	AsyncBlock.queue = TaskQueue ? TaskQueue : FGDKAsyncTaskQueue::GetGenericQueue();
	AsyncBlock.context = this;

	// wrap the delegate with AsyncBlock callback
	if (Delegate)
	{
		AsyncBlock.callback = [](XAsyncBlock* InnerBlock)
		{
			// get our wrapper object
			((FGDKAsyncBlock*)InnerBlock->context)->DelegateWrapper();
		};
	}
}



FGDKAsyncBlock::~FGDKAsyncBlock()
{
	if (XAsyncGetStatus(&AsyncBlock, false) == E_PENDING)
	{
		UE_LOGF(LogGDK, Warning, "[FGDKAsyncBlock::~FGDKAsyncBlock] AsyncBlock still pending for task. Cancelling and firing callback.");
		XAsyncCancel(&AsyncBlock);
	}
}

void FGDKAsyncBlock::DelegateWrapper()
{
	// call the delegate
	if (Delegate)
	{
		Delegate(this);
	}
}

XAsyncBlock* FGDKAsyncBlock::GetInnerBlockForGDKAPI()
{
	return &AsyncBlock;
}

void* FGDKAsyncBlock::GetUserData()
{
	return UserData;
}

HRESULT FGDKAsyncBlock::GetStatus()
{
	return XAsyncGetStatus(&AsyncBlock, false);
}





class FThreadPoolTaskQueue : public FGDKAsyncTaskQueue
{
public:
	FThreadPoolTaskQueue() 
		: FGDKAsyncTaskQueue(XTaskQueueDispatchMode::ThreadPool, XTaskQueueDispatchMode::Manual)
	{
		// set up to dispatch completions once a frame on game thread
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float)
		{
			XTaskQueueDispatch(Queue, XTaskQueuePort::Completion, 0);
			return true;
		}));
	}

	~FThreadPoolTaskQueue()
	{
		FTSTicker::GetCoreTicker().RemoveTicker(TickHandle);
	}

private:
	FTSTicker::FDelegateHandle TickHandle;
};

class FThreadPoolSerialTaskQueue : public FGDKAsyncTaskQueue
{
public:
	FThreadPoolSerialTaskQueue()
		: FGDKAsyncTaskQueue(XTaskQueueDispatchMode::Manual, XTaskQueueDispatchMode::Manual)
	{
		// set up to dispatch completions once a frame on game thread
		TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float)
			{
				// Try to launch one completion task per frame as done in FThreadPoolSerialTaskQueue, if the game thread queue is empty we launch a work task in the thread pool.
				if (!XTaskQueueDispatch(Queue, XTaskQueuePort::Completion, 0)) 
				{
					if (!WorkTasksPipe.HasWork())
					{
						WorkTasksPipe.Launch(
							TEXT("GDK Dispatch queue bg task"),
							[this]
							{
								XTaskQueueDispatch(Queue, XTaskQueuePort::Work, 0);
							},
							UE::Tasks::ETaskPriority::Normal
						);
					}
				}

				return true;
			}));
	}

	~FThreadPoolSerialTaskQueue()
	{
		FTSTicker::GetCoreTicker().RemoveTicker(TickHandle);
	}

private:
	FTSTicker::FDelegateHandle TickHandle;
	UE::Tasks::FPipe WorkTasksPipe{TEXT("GDK Serial Task Queue")};
};

class FBackgroundTaskQueue : public FGDKAsyncTaskQueue
{
public:
	FBackgroundTaskQueue() 
		: FGDKAsyncTaskQueue(XTaskQueueDispatchMode::ThreadPool, XTaskQueueDispatchMode::ThreadPool)
	{
	}
};





static bool bIsTornDown = false;
static TUniquePtr<FThreadPoolTaskQueue> GThreadPoolTaskQueue;
static TUniquePtr<FThreadPoolSerialTaskQueue> GThreadPoolSerialTaskQueue;
static TUniquePtr<FBackgroundTaskQueue> GBackgroundTaskQueue;

void FGDKAsyncTaskQueue::PlatformInit()
{
	bIsTornDown = false;
}

void FGDKAsyncTaskQueue::PlatformTeardown()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XTaskQueueTerminate, XTaskQueueGetCurrentProcessTaskQueue and XTaskQueueSetCurrentProcessTaskQueue are not safe to call on time-sensitive threads

	check(!bIsTornDown);
	bIsTornDown = true;

	// destroy our task queues
	GThreadPoolTaskQueue.Reset();
	GThreadPoolSerialTaskQueue.Reset();
	GBackgroundTaskQueue.Reset();

	// close the system task queue and prevent anyone from using it
	XTaskQueueHandle SystemTaskQueue;
	if (XTaskQueueGetCurrentProcessTaskQueue(&SystemTaskQueue) && SystemTaskQueue != nullptr)
	{
		XTaskQueueTerminate(SystemTaskQueue, true, nullptr, nullptr);
		XTaskQueueCloseHandle(SystemTaskQueue);
	}
	CA_SUPPRESS(6387) // first parameter is marked _In_ but documentation says nullptr is allowed to disable the proces default task queue
	XTaskQueueSetCurrentProcessTaskQueue(nullptr);
}

XTaskQueueHandle FGDKAsyncTaskQueue::GetGenericQueue()
{
	check(!bIsTornDown);

	if (!GThreadPoolTaskQueue.IsValid())
	{
		GThreadPoolTaskQueue = MakeUnique<FThreadPoolTaskQueue>();
	}

	return GThreadPoolTaskQueue->GetQueue();
}

XTaskQueueHandle FGDKAsyncTaskQueue::GetGenericSerialQueue()
{
	check(!bIsTornDown);

	if (!GThreadPoolSerialTaskQueue.IsValid())
	{
		GThreadPoolSerialTaskQueue = MakeUnique<FThreadPoolSerialTaskQueue>();
	}

	return GThreadPoolSerialTaskQueue->GetQueue();
}

XTaskQueueHandle FGDKAsyncTaskQueue::GetBackgroundTaskQueue()
{
	check(!bIsTornDown);

	if (!GBackgroundTaskQueue.IsValid())
	{
		GBackgroundTaskQueue = MakeUnique<FBackgroundTaskQueue>();
	}

	return GBackgroundTaskQueue->GetQueue();
}






FGDKAsyncTaskQueue::FGDKAsyncTaskQueue( XTaskQueueDispatchMode InWorkDispatchMode, XTaskQueueDispatchMode InCompletionDispatchMode )
	: WorkDispatchMode(InWorkDispatchMode)
	, CompletionDispatchMode(InCompletionDispatchMode)
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XTaskQueueCreate is not safe to call on time-sensitive threads

	HRESULT hResult = XTaskQueueCreate(WorkDispatchMode, CompletionDispatchMode, &Queue);
	checkf(SUCCEEDED(hResult), TEXT("XTaskQueueCreate failed: 0x%x"), hResult);
}

FGDKAsyncTaskQueue::~FGDKAsyncTaskQueue()
{
	CancelPendingTasksAndDestroyQueue();
}

void FGDKAsyncTaskQueue::CancelPendingTasksAndDestroyQueue()
{
	if (Queue != nullptr)
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // XTaskQueueTerminate is not safe to call on time-sensitive threads

		// add a terminiation marker into the queue that sets an event when it has completed
		FEvent* QueueTerminated = FPlatformProcess::GetSynchEventFromPool(false);
		XTaskQueueTerminate(Queue, false, (void*)QueueTerminated, []( void* Context)
		{
			((FEvent*)Context)->Trigger();
		});

		// keep dispatching the queue until it has terminated
		do
		{
			// dispatch any outstanding work
			if (WorkDispatchMode == XTaskQueueDispatchMode::Manual)
			{
				XTaskQueueDispatch(Queue, XTaskQueuePort::Work, INFINITE);
			}

			// dispatch any outstanding completion - including the termination callback
			if (CompletionDispatchMode == XTaskQueueDispatchMode::Manual)
			{
				XTaskQueueDispatch(Queue, XTaskQueuePort::Completion, INFINITE);
			}

		} while (!QueueTerminated->Wait(0));

		// clean up
		FPlatformProcess::ReturnSynchEventToPool(QueueTerminated);
		XTaskQueueCloseHandle(Queue);
		Queue = nullptr;
	}
}

HRESULT FGDKAsyncTaskQueue::BlockUntilComplete( XAsyncBlock& AsyncBlock ) const
{
	check(Queue);

	HRESULT hResult;

	// wait until the block is complete, with special handling for game thread
	while ((hResult = XAsyncGetStatus(&AsyncBlock, false)) == E_PENDING)
	{
		if (IsInGameThread())
		{
			FPlatformMisc::PumpMessagesOutsideMainLoop();
		}

		// dispatch any outstanding work (NB. attempting a dispatch on a non-Manual queue will result in an XError callback for illegal access)
		if (WorkDispatchMode == XTaskQueueDispatchMode::Manual)
		{
			XTaskQueueDispatch(Queue, XTaskQueuePort::Work, 0);
		}

		// dispatch any outstanding completion
		if (CompletionDispatchMode == XTaskQueueDispatchMode::Manual)
		{
			XTaskQueueDispatch(Queue, XTaskQueuePort::Completion, 0);
		}
	}

	return hResult;
}










FGDKLocalTaskBlock::FGDKLocalTaskBlock()
	: AsyncBlock(XAsyncBlock{ 0 })
	, Queue(XTaskQueueDispatchMode::ThreadPool, XTaskQueueDispatchMode::Manual)
{
	AsyncBlock.queue = Queue.GetQueue();
}

FGDKLocalTaskBlock::~FGDKLocalTaskBlock()
{
	check(XAsyncGetStatus(&AsyncBlock, false) != E_PENDING);
}

HRESULT FGDKLocalTaskBlock::BlockUntilComplete()
{
	return Queue.BlockUntilComplete(AsyncBlock);
}

HRESULT FGDKLocalTaskBlock::GetStatus()
{
	return XAsyncGetStatus(&AsyncBlock, false);
}





bool FGDKAsyncTaskMonitor::TryCancel(bool bWait)
{
	TSharedPtr<FGDKAsyncBlock> PinnedBlock = WeakBlock.Pin();
	if (PinnedBlock.IsValid())
	{
		XAsyncBlock* AsyncBlock = PinnedBlock->GetInnerBlockForGDKAPI();
		HRESULT hResult = XAsyncGetStatus(AsyncBlock, false);
		if (hResult == E_PENDING)
		{
			XAsyncCancel(AsyncBlock);

			if (bWait)
			{
				while (hResult == E_PENDING)
				{
					if (IsInGameThread())
					{
						FPlatformMisc::PumpMessagesOutsideMainLoop();
					}

					hResult = XAsyncGetStatus(AsyncBlock, false);
				}
			}

			return true;
		}
	}

	return false;
}

bool FGDKAsyncTaskMonitor::IsValid() const
{
	return WeakBlock.IsValid() && WeakBlock.Pin().IsValid();
}


void Internal_InitAsyncTaskMonitor( FGDKAsyncTaskMonitor& TaskMonitor, TSharedPtr<FGDKAsyncBlock> Block )
{
	TaskMonitor.WeakBlock = Block;
}





HRESULT AsyncGDKTask( TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction, XTaskQueueHandle TaskQueue )
{
	FGDKAsyncTaskMonitor UnusedMonitor;
	return AsyncGDKTask( UnusedMonitor, MoveTemp(InitFunction), MoveTemp(ResultFunction), TaskQueue );
}

HRESULT AsyncGDKTask( FGDKAsyncTaskMonitor& OutAsyncTaskMonitor, TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction, XTaskQueueHandle TaskQueue )
{
	// sanity check parameters
	if (InitFunction == nullptr)
	{
		Internal_InitAsyncTaskMonitor( OutAsyncTaskMonitor, nullptr );
		return ERROR_INVALID_PARAMETER;
	}

	// callback context
	struct FContext
	{
		TSharedPtr<FGDKAsyncBlock> TaskBlock;
	};
	FContext* Context = new FContext();

	// completion callback
	auto OnGDKTaskComplete = [ResultFunction]( FGDKAsyncBlock* TaskBlock )
	{
		FContext* Context = (FContext*)TaskBlock->GetUserData();
		check(TaskBlock == Context->TaskBlock.Get());

		if (ResultFunction)
		{
			ResultFunction( *TaskBlock );
		}

		delete Context;
	};

	// create the async task block and try to get started
	Context->TaskBlock = MakeShared<FGDKAsyncBlock>(Context, OnGDKTaskComplete, TaskQueue);
	Internal_InitAsyncTaskMonitor( OutAsyncTaskMonitor, Context->TaskBlock );

	HRESULT hResult = InitFunction( Context->TaskBlock->GetInnerBlockForGDKAPI() );
	if (FAILED(hResult))
	{
		delete Context;
	}

	return hResult;
}



HRESULT LocalGDKTask( TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<HRESULT(XAsyncBlock*)> ResultFunction )
{
	// sanity check parameters
	if (InitFunction == nullptr)
	{
		return ERROR_INVALID_PARAMETER;
	}

	// run the local task
	FGDKLocalTaskBlock Block;
	HRESULT hResult = InitFunction(Block);
	if (SUCCEEDED(hResult))
	{
		hResult = Block.BlockUntilComplete();
		if (SUCCEEDED(hResult) && ResultFunction)
		{
			hResult = ResultFunction(Block);
		}
	}

	return hResult;
}



namespace GDKPrivate
{
	class FGDKTaskContext
	{
	public:
		FGDKTaskContext();
		void Init(TUniqueFunction<HRESULT(XAsyncBlock*)> InitFunction);
		HRESULT Complete(TUniqueFunction<HRESULT(XAsyncBlock*)> ResultFunction);
		UE::Tasks::FTaskEvent AsyncCompleted;

		HRESULT hResult;
		TSharedPtr<FGDKAsyncBlock> AsyncBlock;
	};


	FGDKTaskContext::FGDKTaskContext()
		: AsyncCompleted(UE_SOURCE_LOCATION)
		, hResult(HRESULT_FROM_WIN32(ERROR_INVALID_FUNCTION))
	{
		// prepare the async task block
		AsyncBlock = MakeShared<FGDKAsyncBlock>( 
			nullptr, 
			[this]( FGDKAsyncBlock* TaskBlock )		
			{
				AsyncCompleted.Trigger();
			},
			FGDKAsyncTaskQueue::GetBackgroundTaskQueue() // always use background task queue: we know the completion callback just sets the trigger, so no need to manage that from the gamethread
		); 
	}

	void FGDKTaskContext::Init(TUniqueFunction<HRESULT(XAsyncBlock*)> InitFunction)
	{
		// init callback - if it fails then trigger immediately
		hResult = InitFunction(AsyncBlock->GetInnerBlockForGDKAPI());
		if (FAILED(hResult))
		{
			AsyncCompleted.Trigger();
		}
	}

	HRESULT FGDKTaskContext::Complete(TUniqueFunction<HRESULT(XAsyncBlock*)> ResultFunction)
	{
		// completion callback - only call if the init callback succeeded
		if (SUCCEEDED(hResult) && ResultFunction)
		{
			hResult = ResultFunction(AsyncBlock->GetInnerBlockForGDKAPI());
		}
		return hResult;
	}
}


FGDKTask LaunchGDKTask(const TCHAR* DebugName, TUniqueFunction<HRESULT(XAsyncBlock*)> InitFunction, TUniqueFunction<HRESULT(XAsyncBlock*)> ResultFunction, UE::Tasks::ETaskPriority Priority)
{
	FGDKAsyncTaskMonitor UnusedMonitor;
	return LaunchGDKTask( DebugName, UnusedMonitor, MoveTemp(InitFunction), MoveTemp(ResultFunction), Priority );
}

FGDKTask LaunchGDKTask(const TCHAR* DebugName, FGDKAsyncTaskMonitor& OutAsyncTaskMonitor, TUniqueFunction<HRESULT(XAsyncBlock*)> InitFunction, TUniqueFunction<HRESULT(XAsyncBlock*)> ResultFunction, UE::Tasks::ETaskPriority Priority)
{
	TUniquePtr<GDKPrivate::FGDKTaskContext> Context = MakeUnique<GDKPrivate::FGDKTaskContext>();
	Internal_InitAsyncTaskMonitor(OutAsyncTaskMonitor, Context->AsyncBlock);

	// run init
	Context->Init(MoveTemp(InitFunction));
	
	// run result task after init & the context completion
	return Launch(DebugName,
		[Context = MoveTemp(Context), ResultFunction = MoveTemp(ResultFunction)]() mutable
		{
			return Context->Complete(MoveTemp(ResultFunction));
		}, 
		Context->AsyncCompleted,
		Priority
	);
}


#endif //WITH_GRDK
