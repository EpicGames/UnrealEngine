// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKThreadCheck.h"

#if WITH_GDK_THREAD_CHECK
#include "Misc/Parse.h"
#include "Misc/CommandLine.h"

#if IS_MONOLITHIC
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <XThread.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

FGDKScopedNotTimeSensitive::FGDKScopedNotTimeSensitive()
{
	bWasTimeSensitive = XThreadIsTimeSensitive();
	XThreadSetTimeSensitive(false);
}

FGDKScopedNotTimeSensitive::~FGDKScopedNotTimeSensitive()
{
	XThreadSetTimeSensitive(bWasTimeSensitive);
}
#endif //IS_MONOLITHIC


bool IsGDKTimeSensitiveThreadCheckEnabled()
{
#if WITH_GDK_THREAD_CHECK_ALWAYS
	return true;

#else
	static bool bTimeSensitiveThreadCheck = FParse::Param(FCommandLine::Get(), TEXT("TimeSensitiveThreadCheck"));
	return bTimeSensitiveThreadCheck;

#endif //WITH_GDK_THREAD_CHECK_ALWAYS
}

#endif //WITH_GDK_THREAD_CHECK
