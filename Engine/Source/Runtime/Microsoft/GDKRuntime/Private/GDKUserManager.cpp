// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKUserManager.h"
#if WITH_GRDK
#include "Misc/MessageDialog.h"
#include "Misc/CoreDelegates.h"
#include "HAL/IConsoleManager.h"
#include "GDKTaskQueueHelpers.h"
#include "GDKHandle.h"
#include "GDKThreadCheck.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Containers/Ticker.h"
#include "Stats/Stats.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
#include <XGameErr.h>
#include <XUser.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

DEFINE_LOG_CATEGORY_STATIC(LogGDKUserManager, Log, All);

static int32 GGDKUserAutoDebugDump = 0;
static FAutoConsoleVariableRef CVarGDKDebugAutoDump(
	TEXT("GDK.AutoDumpUsers"),
	GGDKUserAutoDebugDump,
	TEXT("Logs out before & after states for all user operations.")
	TEXT("0 = disabled, 1 = after only, 2 = before & after"),
	ECVF_Default
);





FGDKUserManager::FGDKUserManager()
{
	// resize the array of seats to accommodate all potential users
	uint32_t MaxUsers = 0;
	HRESULT Result = XUserGetMaxUsers(&MaxUsers);
	UE_CLOGF(FAILED(Result), LogGDKUserManager, Warning, "FGDKUserManager::Init - XUserGetMaxUsers - Error: (0x%0.8X).", Result);
	Seats.AddDefaulted(MaxUsers + 1); //allowing one extra seat to hold all unpaired devices
	
	// add callbacks
	{
		auto UserChangeFn = [](void* Context, XUserLocalId UserLocalId, XUserChangeEvent Event)
		{
			((FGDKUserManager*)Context)->OnUserChange(UserLocalId, Event);
		};
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XUserRegisterForChangeEvent is not safe to call on time-sensitive threads
		XUserRegisterForChangeEvent(FGDKAsyncTaskQueue::GetGenericQueue(), this, UserChangeFn, &UserChangeCallbackToken);
	}

	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FGDKUserManager::OnApplicationWillEnterBackground);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FGDKUserManager::OnApplicationHasEnteredForeground);	
}



FGDKUserManager::~FGDKUserManager()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) XUserUnregisterForChangeEvent & XUserUnregisterForDeviceAssociationChanged are not safe to call on time-sensitive threads

	FCoreDelegates::ApplicationWillEnterBackgroundDelegate.RemoveAll(this);
	FCoreDelegates::ApplicationHasEnteredForegroundDelegate.RemoveAll(this);

	XUserUnregisterForChangeEvent(UserChangeCallbackToken, true);

	{
		FScopeLock Lock(&StateLock);
		Seats.Reset();
	}
}



bool FGDKUserManager::HasDefaultUser() const
{
	// default user will always be user 0
	return Seats[0].PairedUserHandle.IsValid();
}

void FGDKUserManager::PickUserAsync(XUserAddOptions Options, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate)
{
	HRESULT hResult = AsyncGDKTask(
		[Options](XAsyncBlock* Block)
		{
			HRESULT hResult = XUserAddAsync(Options, Block);
			UE_CLOGF(FAILED(hResult), LogGDKUserManager, Warning, "XUserAddAsync failed: 0x%0.8X", hResult);
			return hResult;
		},
		[this, Options, OnPickUserCompleteDelegate] (XAsyncBlock* Block)
		{
			FGDKUserHandle User;
			HRESULT hResult = XUserAddResult(Block, User.GetInitReference());
			UE_CLOGF(FAILED(hResult) && (hResult != E_ACCESSDENIED), LogGDKUserManager, Warning, "XUserAddResult failed: 0x%0.8X", hResult);
			HandleXUserAddComplete(Options, hResult, User, OnPickUserCompleteDelegate);
		}
	);

	if (FAILED(hResult))
	{
		PickUserAsyncComplete(hResult, FGDKUserHandle(), OnPickUserCompleteDelegate);
	}
}

void FGDKUserManager::HandleXUserAddComplete(XUserAddOptions Options, HRESULT hResult, FGDKUserHandle User, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate)
{
	if (hResult == E_ACCESSDENIED)
	{
		PickUserAsync(Options, OnPickUserCompleteDelegate);
		return;
	}

	if (SUCCEEDED(hResult))
	{
		uint64 UserId;
		hResult = XUserGetId(User, &UserId);
		UE_CLOGF(FAILED(hResult), LogGDKUserManager, Warning, "XUserGetId failed: 0x%0.8X", hResult);
	}

	if (hResult == E_GAMEUSER_RESOLVE_USER_ISSUE_REQUIRED && !FApp::IsUnattended())
	{
		ResolveUserIssueWithUiAsync(User, OnPickUserCompleteDelegate);
		return;
	}

	PickUserAsyncComplete(hResult, User, OnPickUserCompleteDelegate);
}

void FGDKUserManager::ResolveUserIssueWithUiAsync(FGDKUserHandle User, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate)
{
	HRESULT hResult = AsyncGDKTask(
		[User](XAsyncBlock* Block)
		{
			HRESULT hResult = XUserResolveIssueWithUiAsync(User, nullptr, Block);
			UE_CLOGF(FAILED(hResult), LogGDKUserManager, Warning, "XUserResolveIssueWithUiAsync failed: 0x%0.8X", hResult);
			return hResult;
		},
		[this, OnPickUserCompleteDelegate, User] (XAsyncBlock* Block)
		{
			HRESULT hResult = XUserResolveIssueWithUiResult(Block);
			UE_CLOGF(FAILED(hResult) && (hResult != E_ACCESSDENIED), LogGDKUserManager, Warning, "XUserResolveIssueWithUiResult failed: 0x%0.8X", hResult);
			HandleResolveUserIssueWithUiComplete(hResult, User, OnPickUserCompleteDelegate);
		}
	);

	if (FAILED(hResult))
	{
		PickUserAsyncComplete(hResult, User, OnPickUserCompleteDelegate);
	}
}

void FGDKUserManager::HandleResolveUserIssueWithUiComplete(HRESULT hResult, FGDKUserHandle User, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate)
{
	if (hResult == E_ACCESSDENIED)
	{
		ResolveUserIssueWithUiAsync(User, OnPickUserCompleteDelegate);
		return;
	}

	PickUserAsyncComplete(hResult, User, OnPickUserCompleteDelegate);
}

void FGDKUserManager::PickUserAsyncComplete(HRESULT hResult, FGDKUserHandle User, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate)
{
	if (User.IsValid() && GetCurrentSeatForUser(User) == INDEX_NONE)
	{
		XUserLocalId UserLocalId;
		if (SUCCEEDED(XUserGetLocalId(User, &UserLocalId)))
		{
			FScopeLock Lock(&StateLock);
			SignedOutXUserLocalIds.Remove(UserLocalId.value);
		}

		AutoDebugDump(TEXT("Before (via PickUser Callback)"), EDebugDumpType::BeforeStateChange);
			
		int32 SeatIndex = FindSeatForUser(User);
		check(SeatIndex != INDEX_NONE);
			
		UE_LOGF(LogGDKUserManager, Log, "New user %ls[%d] signed in via callback", *LexToString(User), SeatIndex);
		AutoDebugDump(TEXT("After (via PickUser Callback)"), EDebugDumpType::AfterStateChange);
	}

	OnPickUserCompleteDelegate.ExecuteIfBound(hResult, User);
}

void FGDKUserManager::AddDefaultUser()
{
	// block and try to silently add the default user
	FGDKUserHandle User;
	HRESULT hResult = LocalGDKTask( 
		[&User]( XAsyncBlock* Block )
		{
			return XUserAddAsync( XUserAddOptions::AddDefaultUserSilently, Block );
		},
		[&User]( XAsyncBlock* Block )
		{
			return XUserAddResult(Block, User.GetInitReference());
		}
	);

	// check initial result
	if (SUCCEEDED(hResult))
	{
		uint64 UserId;
		hResult = XUserGetId(User, &UserId);
		UE_CLOGF(FAILED(hResult), LogGDKUserManager, Warning, "Failed to get xuid for default user");
	}

	// check final result
	if (SUCCEEDED(hResult))
	{
		ensure(FindSeatForUser(User) != INDEX_NONE);
	}
	else if (hResult == E_GAMEUSER_RESOLVE_USER_ISSUE_REQUIRED && !FApp::IsUnattended())
	{
		// cannot block & show resolve UI, so start an async task instead
		ResolveUserIssueWithUiAsync(User, FGDKPickUserCompleteDelegate::CreateLambda([this]
			(HRESULT hResult, FGDKUserHandle User)
			{
				ensure(FindSeatForUser(User) != INDEX_NONE);
			}
		));
	}
	else
	{
		//NB. if you see E_ABORT here, and the Xbox live login dialog box gave error code 0x87DD0005 it probably means this Device Family isn't enabled for Xbox Live in Partner Center -> Xbox Live -> Xbox Live Settings
		UE_LOGF(LogGDKUserManager, Warning, "Failed to add default user. Error 0x%X", hResult);
		UE_CLOGF((hResult == E_GAMEUSER_NO_DEFAULT_USER),             LogGDKUserManager, Warning, "No default user.");
		UE_CLOGF((hResult == E_GAMEUSER_NO_PACKAGE_IDENTITY),         LogGDKUserManager, Warning, "No package identity");
		UE_CLOGF((hResult == E_GAMEUSER_RESOLVE_USER_ISSUE_REQUIRED), LogGDKUserManager, Warning, "Not allowed to display user resolve UI." );
	}
}





FGDKUserHandle FGDKUserManager::GetUserHandleByPlatformId(FPlatformUserId PlatformUserId) const
{
	int32 SeatIndex = PlatformUserId;
	if (!Seats.IsValidIndex(SeatIndex))
	{
		return FGDKUserHandle();
	}

	return Seats[SeatIndex].PairedUserHandle;
}


FPlatformUserId FGDKUserManager::GetPlatformIdForSeat(int32 SeatIndex) const
{
	if (Seats.IsValidIndex(SeatIndex))
	{
		return FPlatformMisc::GetPlatformUserForUserIndex(SeatIndex);
	}
	
	return PLATFORMUSERID_NONE;
}


FGDKUserHandle FGDKUserManager::GetUserHandleByXUserId(uint64 XUserId) const
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XUserFindUserById is not safe to call on a time-sensitive thread

	FGDKUserHandle UserHandle;
	XUserFindUserById(XUserId, UserHandle.GetInitReference());
	return UserHandle;
}


TOptional<uint64> FGDKUserManager::GetXUserIdByPlatformId(FPlatformUserId PlatformUserId) const
{
	FGDKUserHandle UserHandle = GetUserHandleByPlatformId(PlatformUserId);

	TOptional<uint64> Result;

	if (UserHandle.IsValid())
	{
		uint64 Xuid;
		if (SUCCEEDED(XUserGetId(UserHandle, &Xuid)))
		{
			Result = Xuid;
		}
	}

	return Result;
}


FPlatformUserId FGDKUserManager::GetPlatformIdByUserHandle(const FGDKUserHandle& UserHandle)
{
	return GetPlatformIdForSeat(FindSeatForUser(UserHandle));
}



int32 FGDKUserManager::FindSeatForUser( const FGDKUserHandle& Handle)
{
	FScopeLock Lock(&StateLock);

	if (!Handle.IsValid())
	{
		return INDEX_NONE;
	}

	int32 SeatIndex = GetCurrentSeatForUser(Handle);
	if (SeatIndex == INDEX_NONE)
	{
		XUserLocalId UserLocalId;
		if (SUCCEEDED(XUserGetLocalId(Handle, &UserLocalId)) && SignedOutXUserLocalIds.Contains(UserLocalId.value))
		{
			UE_LOGF(LogGDKUserManager, Log, "Will not assign seat to signed-out user %ls", *LexToString(Handle) );
			return INDEX_NONE;
		}

		AutoDebugDump(TEXT("Before AssignSeatForUser"), EDebugDumpType::BeforeStateChange);

		// we have not heard about this user before so assign them a seat now
		SeatIndex = FindEmptySeat();
		check(SeatIndex != INDEX_NONE);
		Seats[SeatIndex].PairedUserHandle = Handle;

		UE_LOGF(LogGDKUserManager, Log, "Assigning user %ls to seat %d", *LexToString(Handle), SeatIndex);
		AutoDebugDump(TEXT("After AssignSeatForUser"), EDebugDumpType::AfterStateChange);

		BroadcastUserAddedOnGameThread(SeatIndex);
	}

	return SeatIndex;
}



int32 FGDKUserManager::GetCurrentSeatForUser(const FGDKUserHandle& Handle) const
{
	if (!Handle.IsValid())
	{
		return INDEX_NONE;
	}

	return Seats.IndexOfByPredicate([Handle](const FSeat& Seat)
	{
		return (Seat.PairedUserHandle.IsValid() && Handle == Seat.PairedUserHandle);
	});
}



int32 FGDKUserManager::FindEmptySeat() const
{
	for( int SeatIndex = 0; SeatIndex < Seats.Num(); SeatIndex++ )
	{
		if (Seats[SeatIndex].PairedUserHandle.IsValid())
		{
			continue;
		}

		return SeatIndex;
	}

	return INDEX_NONE;
}


TArray<FGDKUserHandle> FGDKUserManager::GetAllUserHandles() const
{
	TArray<FGDKUserHandle> Result;
	for( int SeatIndex = 0; SeatIndex < Seats.Num(); SeatIndex++ )
	{
		if (Seats[SeatIndex].PairedUserHandle.IsValid())
		{
			Result.Add( Seats[SeatIndex].PairedUserHandle );
		}
	}

	return MoveTemp(Result);
}


bool FGDKUserManager::WasUserRecentlySignedOut(XUserLocalId UserLocalId) const
{
	return SignedOutXUserLocalIds.Contains(UserLocalId.value);
}


void FGDKUserManager::OnUserChange(XUserLocalId UserLocalId, XUserChangeEvent Event)
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // XUserFindUserByLocalId is not safe to call on a time-sensitive thread

	FGDKUserHandle UserHandle;
	XUserFindUserByLocalId(UserLocalId, UserHandle.GetInitReference());

	if (Event == XUserChangeEvent::SignedInAgain )
	{
		// do nothing here: seat assignment & signout state should handled in PickUserAsyncComplete for consistency with genuine new user flow
	}
	else if (Event == XUserChangeEvent::SigningOut)
	{
		{
			FScopeLock Lock(&StateLock);
			SignedOutXUserLocalIds.Add(UserLocalId.value);
		}

		int32 SeatIndex = GetCurrentSeatForUser(UserHandle);
		if (SeatIndex != INDEX_NONE)
		{
			UE_LOGF(LogGDKUserManager, Log, "User %ls[%d] signing out via callback", *LexToString(UserHandle), SeatIndex);
			RemoveUserOnGameThread(SeatIndex);
		}
	}
}


void FGDKUserManager::BroadcastUserAddedOnGameThread(int32 SeatIndex)
{
	if (IsInGameThread())
	{
		// we are on the gamethread - signal the delegate directly
		FCoreDelegates::OnUserLoginChangedEvent.Broadcast(true, GetPlatformIdForSeat(SeatIndex), SeatIndex);
		UE_CLOGF( GGDKUserAutoDebugDump != 0, LogGDKUserManager, Log, "\n**** FCoreDelegates::OnUserLoginChangedEvent - User %d Login ****\n", SeatIndex );
	}
	else 
	{
		// we are not on the gamethread - request a callback when we are
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [SeatIndex, this]()
		{
			BroadcastUserAddedOnGameThread(SeatIndex);
		});
	}

}

void FGDKUserManager::RemoveUserOnGameThread(int32 SeatIndex)
{
	if (IsInGameThread())
	{
		// we are on the gamethread - remove the user
		AutoDebugDump(TEXT("Before User Remove"), EDebugDumpType::BeforeStateChange);
		UE_LOGF(LogGDKUserManager, Log, "User %ls (%d) signed out", *LexToString(Seats[SeatIndex].PairedUserHandle), SeatIndex);

		FCoreDelegates::OnUserLoginChangedEvent.Broadcast(false, GetPlatformIdForSeat(SeatIndex), SeatIndex);
		UE_CLOGF( GGDKUserAutoDebugDump != 0, LogGDKUserManager, Log, "\n**** FCoreDelegates::OnUserLoginChangedEvent - User %d Logout ****\n", SeatIndex );

		{
			FScopeLock Lock(&StateLock);
			Seats[SeatIndex].PairedUserHandle.Clear();
		}

		AutoDebugDump(TEXT("After User Remove"), EDebugDumpType::AfterStateChange);
	}
	else
	{
		// we are not on the gamethread - request a callback when we are
		ExecuteOnGameThread(UE_SOURCE_LOCATION, [SeatIndex, this]()
		{
			RemoveUserOnGameThread(SeatIndex);
		});
	}
}


void FGDKUserManager::OnApplicationWillEnterBackground()
{
	SCOPED_ENTER_BACKGROUND_EVENT(STAT_FGDKUserManager_OnApplicationWillEnterBackground);
	AutoDebugDump(TEXT("OnApplicationWillEnterBackground"), EDebugDumpType::AfterStateChange);
}

void FGDKUserManager::OnApplicationHasEnteredForeground()
{
	AutoDebugDump(TEXT("OnApplicationHasEnteredForeground"), EDebugDumpType::AfterStateChange);
}




void FGDKUserManager::DebugDump( const TCHAR* BannerMessage, EDebugDumpType::Type Type ) const
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (debug only) XUserFindForDevice isn't safe to call on time-sensitive threads

	// write banner
	UE_LOGF( LogGDKUserManager, Log, "*** GDK User Manager : %ls ***", BannerMessage ? BannerMessage : TEXT("") );

	// helper lambda to write out empty spans
	int EmptyRangeStart = INDEX_NONE;
	auto WriteEmptyItems = [&EmptyRangeStart]( int SeatIndex )
	{
		if (EmptyRangeStart != INDEX_NONE)
		{
			if (EmptyRangeStart == SeatIndex - 1)
			{
				UE_LOGF(LogGDKUserManager, Log, "\t\t%d - empty", EmptyRangeStart);
			}
			else
			{
				UE_LOGF(LogGDKUserManager, Log, "\t\t%d-%d : empty", EmptyRangeStart, SeatIndex - 1);
			}
			EmptyRangeStart = INDEX_NONE;
		}
	};

	// write out all users
	for( int SeatIndex = 0; SeatIndex < Seats.Num(); SeatIndex++ )
	{
		const FSeat& Seat = Seats[SeatIndex];
		bool bIsEmpty = !Seat.PairedUserHandle.IsValid();

		bool bWasEmpty = (EmptyRangeStart != INDEX_NONE);
		if (bWasEmpty != bIsEmpty)
		{
			if (bWasEmpty)
			{
				WriteEmptyItems(SeatIndex);
			}
			else
			{
				EmptyRangeStart = SeatIndex;
			}
		}
		if (!bIsEmpty)
		{
			UE_LOGF(LogGDKUserManager, Log, "\t\t%d : %ls", SeatIndex, *LexToString(Seat.PairedUserHandle));
		}
	}
	WriteEmptyItems(Seats.Num());

	// check users
	for( int SeatIndex = 0; SeatIndex < Seats.Num(); SeatIndex++ )
	{
		const FSeat& Seat = Seats[SeatIndex];
		if (Seat.PairedUserHandle.IsValid())
		{
			XUserState State;
			HRESULT Result = XUserGetState(Seat.PairedUserHandle, &State);
			if (SUCCEEDED(Result))
			{
				UE_CLOGF(State != XUserState::SignedIn, LogGDKUserManager, Warning, "User in seat %d is not signed in (%d)", SeatIndex, EnumToUnderlyingType(State));
			}
			else if (Result == E_GAMERUNTIME_SUSPENDED)
			{
				//... do nothing - cannot query user state while suspended
			}
			else
			{
				UE_LOGF(LogGDKUserManager, Warning, "Failed to get user state for user in seat %d : HR=0x%X", SeatIndex, Result );
			}
		}
	}

	// done
	UE_LOGF( LogGDKUserManager, Log, "****************************************************\n" );

}

 void FGDKUserManager::DebugDump() const
 {
	 DebugDump( TEXT("Debug Dump") );
 }

void FGDKUserManager::AutoDebugDump( const TCHAR* BannerMessage, EDebugDumpType::Type Type )
{
	bool bAfterStateChange = (Type & EDebugDumpType::AfterStateChange) != 0;

	if (GGDKUserAutoDebugDump && ( GGDKUserAutoDebugDump == 2 || bAfterStateChange ) ) // 0 = disabled, 1 = after only, 2 = both
	{
		DebugDump(BannerMessage, Type );
	}
}


#endif //WITH_GRDK
