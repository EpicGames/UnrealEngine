// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDKRuntimeModule.h"
#if WITH_GRDK
#include "CoreMinimal.h"
#include "GDKHandle.h"
#include "HAL/CriticalSection.h"
#include "Containers/Set.h"
#include "Misc/Optional.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <XUser.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#define UE_API GDKRUNTIME_API

/*
 * The GDK user manager is responsible for assigning a suitable FPlatformUserId for GDK users. FPlatformUserIds are currently recycled when a user signs out & is replaced by another and represent the seat that the user is assigned to
 */
class FGDKUserManager
{
public:
	UE_API FGDKUserManager();
	UE_API virtual ~FGDKUserManager();

	UE_API FGDKUserHandle GetUserHandleByPlatformId(FPlatformUserId PlatformUserId) const;
	UE_API FGDKUserHandle GetUserHandleByXUserId(uint64 XUserId) const;
	UE_API FPlatformUserId GetPlatformIdByUserHandle(const FGDKUserHandle& UserHandle);
	UE_API TOptional<uint64> GetXUserIdByPlatformId(FPlatformUserId PlatformUserId) const;
	UE_API TArray<FGDKUserHandle> GetAllUserHandles() const;
	UE_API bool WasUserRecentlySignedOut(XUserLocalId UserLocalId) const;
	UE_API int32 FindEmptySeat() const;
	UE_API bool HasDefaultUser() const;
	UE_API void AddDefaultUser();
	UE_API void PickUserAsync(XUserAddOptions Options, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate);

	UE_API void DebugDump() const;

protected:
	// internal helper functions
	UE_API FPlatformUserId GetPlatformIdForSeat(int32 SeatIndex) const;
	UE_API int32 FindSeatForUser(const FGDKUserHandle& UserHandle);
	UE_API int32 GetCurrentSeatForUser(const FGDKUserHandle& UserHandle) const;

private:
	// callbacks
	void OnUserChange(XUserLocalId UserLocalId, XUserChangeEvent Event);
	void OnApplicationWillEnterBackground();
	void OnApplicationHasEnteredForeground();

	// helper functions
	FORCEINLINE static bool IsValid(const XUserLocalId& UserLocalId) { return UserLocalId.value != 0; }

	void HandleXUserAddComplete(XUserAddOptions Options, HRESULT hResult, FGDKUserHandle User, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate);
	void ResolveUserIssueWithUiAsync(FGDKUserHandle User, FGDKPickUserCompleteDelegate OnResolveUserIssueWithUiComplete);
	void HandleResolveUserIssueWithUiComplete(HRESULT hResult, FGDKUserHandle User, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate);
	void PickUserAsyncComplete(HRESULT hResult, FGDKUserHandle User, FGDKPickUserCompleteDelegate OnPickUserCompleteDelegate);

	struct EDebugDumpType
	{
		typedef uint8 Type;
		static const uint8 BeforeStateChange = 1;
		static const uint8 AfterStateChange = 2;
	};
	void AutoDebugDump( const TCHAR* BannerMessage, EDebugDumpType::Type Type );
	void DebugDump( const TCHAR* BannerMessage, EDebugDumpType::Type Type = 0 ) const;

	void BroadcastUserAddedOnGameThread( int32 SeatIndex );
	void RemoveUserOnGameThread( int32 SeatIndex );

	struct FSeat
	{
		FGDKUserHandle PairedUserHandle;
	};
	TArray<FSeat>               Seats;
	TSet<uint64>                SignedOutXUserLocalIds;
	FCriticalSection            StateLock;
	XTaskQueueRegistrationToken UserChangeCallbackToken;
};

#undef UE_API

#endif //WITH_GRDK
