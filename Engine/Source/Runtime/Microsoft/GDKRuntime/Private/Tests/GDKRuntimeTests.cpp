// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKRuntimeModule.h"
#include "Misc/AutomationTest.h"
#if WITH_GRDK && WITH_DEV_AUTOMATION_TESTS
#include "GDKTaskQueueHelpers.h"
#include "HAL/PlatformProcess.h"
#include "HAL/Event.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameUI.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

// MyGame -ExecCmds="Automation RunTests GDKTests" -TestExit="Automation Test Queue Empty" -unattended
namespace GDKTests
{
	struct FTestContext
	{
		FTestContext(FAutomationTestBase* InTest) : Test(InTest) {}
		FAutomationTestBase* Test;
	};


	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER(FGDKTestCmd_Wait, float, Duration); // NB. not using FWaitLatentCommand because it's in Engine and we're in Core
	bool FGDKTestCmd_Wait::Update()
	{
		const double NewTime = FPlatformTime::Seconds();
		if (NewTime - StartTime >= Duration)
		{
			return true;
		}
		return false;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER( FGDKTestCmd_WaitForEvent, FAutomationTestBase*, Test, FEvent*, Event );
	bool FGDKTestCmd_WaitForEvent::Update()
	{
		return Event->Wait(FTimespan::FromSeconds(0.1f));
		
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_THREE_PARAMETER( FGDKTestCmd_CheckResult, FAutomationTestBase*, Test, HRESULT, hResult, HRESULT, hResultExpected );
	bool FGDKTestCmd_CheckResult::Update()
	{
		if (hResultExpected != hResult)
		{
			Test->AddError(FString::Printf(TEXT("Expected hResult 0x%X but got 0x%X"), hResultExpected, hResult ));
		}
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER( FGDKTestCmd_RunOp, TFunction<void(void)>, Op );
	bool FGDKTestCmd_RunOp::Update()
	{
		Op();
		return true;
	}



	struct FTaskQueueTestContext : public FTestContext
	{
		FTaskQueueTestContext(FAutomationTestBase* InTest) 
			: FTestContext(InTest)
			, Event(FPlatformProcess::GetSynchEventFromPool(false))
			, hResult(E_NOTIMPL)
		{
		}

		~FTaskQueueTestContext()
		{
			FPlatformProcess::ReturnSynchEventToPool(Event);
			TaskMonitor.TryCancel(false);
		}

		// NB. some redudent members depending on the test
		FEvent* Event;
		HRESULT hResult;
		FGDKAsyncTaskMonitor TaskMonitor;
		FGDKTask Task;
	};



	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER( FGDKTestCmd_TaskQueue_Cancel, FTaskQueueTestContext*, TestContext );
	bool FGDKTestCmd_TaskQueue_Cancel::Update()
	{
		bool bCancelResult = TestContext->TaskMonitor.TryCancel(true);
		TestContext->Test->TestTrue("task cancel success", bCancelResult);
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER( FGDKTestCmd_TaskQueue_WaitForTask, FTaskQueueTestContext*, TestContext );
	bool FGDKTestCmd_TaskQueue_WaitForTask::Update()
	{
		return TestContext->Task.Wait(FTimespan::FromSeconds(0.1f));
	}


	DEFINE_LATENT_AUTOMATION_COMMAND_TWO_PARAMETER( FGDKTestCmd_TaskQueue_CheckResult, FTaskQueueTestContext*, TestContext, HRESULT, hResultExpected );
	bool FGDKTestCmd_TaskQueue_CheckResult::Update()
	{
		HRESULT hResult = TestContext->Task.GetResult();
		if (hResultExpected != hResult)
		{
			TestContext->Test->AddError(FString::Printf(TEXT("Expected hResult 0x%X but got 0x%X"), hResultExpected, hResult ));
		}
		return true;
	}

	DEFINE_LATENT_AUTOMATION_COMMAND_ONE_PARAMETER( FGDKTestCmd_TaskQueue_CleanUp, FTaskQueueTestContext*, TestContext );
	bool FGDKTestCmd_TaskQueue_CleanUp::Update()
	{
		delete TestContext;
		return true;
	}


	// run an async task and then attempt to cancel it from the game thread. verifies the completion callback fires and returns E_ABORT
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGDKTaskQueueTestCancel, "GDKTests.TaskQueue.CancelAsyncGDKTask", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter )
	bool FGDKTaskQueueTestCancel::RunTest(const FString& Parameters)
	{
		FTaskQueueTestContext* TestContext = new FTaskQueueTestContext(this);

		auto Op = [TestContext]()
		{
			AsyncGDKTask( 
				TestContext->TaskMonitor,
				[TestContext]( XAsyncBlock* Block )
				{
					return XGameUiShowMessageDialogAsync( Block, TCHAR_TO_UTF8(*TestContext->Test->GetTestFullName()), "Should Be Cancelled Automatically", "Test", "Test", "Test", XGameUiMessageDialogButton::First, XGameUiMessageDialogButton::Second );
				},
				[TestContext]( XAsyncBlock* Block )
				{
					XGameUiMessageDialogButton Button;
					TestContext->hResult = XGameUiShowMessageDialogResult(Block, &Button);
					TestContext->Event->Trigger();
				}
			);
		};

		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_RunOp(Op));
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_Wait(1.0f) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_TaskQueue_Cancel(TestContext) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_WaitForEvent(this, TestContext->Event) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_CheckResult(this, TestContext->hResult, E_ABORT) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_TaskQueue_CleanUp(TestContext) );

		return true;
	}


	// run an async task and then attempt to cancel it in the background. verifies the completion callback fires and returns E_ABORT
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGDKTaskQueueTestCancelFromBgThread, "GDKTests.TaskQueue.BgCancelAsyncGDKTask", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter )
	bool FGDKTaskQueueTestCancelFromBgThread::RunTest(const FString& Parameters)
	{
		FTaskQueueTestContext* TestContext = new FTaskQueueTestContext(this);

		auto Op = [TestContext]()
		{
			AsyncGDKTask( 
				TestContext->TaskMonitor,
				[TestContext]( XAsyncBlock* Block )
				{
					return XGameUiShowMessageDialogAsync( Block, TCHAR_TO_UTF8(*TestContext->Test->GetTestFullName()), "Should Be Cancelled Automatically", "Test", "Test", "Test", XGameUiMessageDialogButton::First, XGameUiMessageDialogButton::Second );
				},
				[TestContext]( XAsyncBlock* Block )
				{
					XGameUiMessageDialogButton Button;
					TestContext->hResult = XGameUiShowMessageDialogResult(Block, &Button);
					TestContext->Event->Trigger();
				}
			);
		};

		auto CancelOpOnBgThread = [TestContext]()
		{
			AsyncTask( ENamedThreads::AnyNormalThreadNormalTask,[TestContext]()
			{
				check(!IsInGameThread());
				bool bCancelResult = TestContext->TaskMonitor.TryCancel(true);
				TestContext->Test->TestEqual("Check cancel success from background thread", bCancelResult, true);
			});
		};


		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_RunOp(Op));
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_Wait(1.0f) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_RunOp(CancelOpOnBgThread) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_WaitForEvent(this, TestContext->Event) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_CheckResult(this, TestContext->hResult, E_ABORT) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_TaskQueue_CleanUp(TestContext) );

		return true;
	}



	// create an async task using the UE::Task system and then attempt to cancel it. verifies the completion callback fires and returns E_ABORT
	IMPLEMENT_SIMPLE_AUTOMATION_TEST(FGDKTaskQueueTestCancelTask, "GDKTests.TaskQueue.CancelLaunchGDKTask", EAutomationTestFlags::ClientContext | EAutomationTestFlags::EngineFilter )
	bool FGDKTaskQueueTestCancelTask::RunTest(const FString& Parameters)
	{
		FTaskQueueTestContext* TestContext = new FTaskQueueTestContext(this);

		auto Op = [TestContext]()
		{
			TestContext->Task = LaunchGDKTask( 
				UE_SOURCE_LOCATION,
				TestContext->TaskMonitor,
				[TestContext]( XAsyncBlock* Block )
				{
					return XGameUiShowMessageDialogAsync( Block, TCHAR_TO_UTF8(*TestContext->Test->GetTestFullName()), "Should Be Cancelled Automatically", "Test", "Test", "Test", XGameUiMessageDialogButton::First, XGameUiMessageDialogButton::Second );
				},
				[TestContext]( XAsyncBlock* Block )
				{
					XGameUiMessageDialogButton Button;
					return XGameUiShowMessageDialogResult(Block, &Button);
				}
			);
		};

		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_RunOp(Op));
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_Wait(1.0f) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_TaskQueue_Cancel(TestContext) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_TaskQueue_WaitForTask(TestContext) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_TaskQueue_CheckResult(TestContext, E_ABORT) );
		ADD_LATENT_AUTOMATION_COMMAND( FGDKTestCmd_TaskQueue_CleanUp(TestContext) );

		return true;
	}
}

#endif //WITH_GRDK && WITH_DEV_AUTOMATION_TESTS

