// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#if WITH_GRDK

#include "Misc/AssertionMacros.h"
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XUser.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

namespace GDKHandleTraits
{
	template<typename T>
	static int32 CompareHandle(T Lhs, T Rhs)
	{
		return (Lhs == Rhs) ? 0 : 1;
	}

	template<typename T>
	static void DuplicateHandle(T Source, T* Target)
	{
	}

	template<typename T>
	static void ReleaseHandle(T Handle)
	{
	}
}

template <typename HandleType>
class TGDKHandle
{
public:
	using ThisType = TGDKHandle<HandleType>;

	TGDKHandle() :
		Handle{ nullptr }
	{
	}
	~TGDKHandle()
	{
		Clear();
	}

	explicit TGDKHandle(HandleType source) :
		Handle{ source }
	{
	}

	TGDKHandle(const ThisType& source) :
		Handle{ nullptr }
	{
		if (source.Handle)
		{
			GDKHandleTraits::DuplicateHandle(source.Handle, &Handle);
		}
	}

	ThisType &operator=(const ThisType& source)
	{
		Clear();

		if (source.Handle)
		{
			GDKHandleTraits::DuplicateHandle(source.Handle, &Handle);
		}
		return *this;
	}

	TGDKHandle(ThisType&& moveFrom) :
		Handle{ moveFrom.Handle }
	{
		moveFrom.Handle = nullptr;
	}

	ThisType& operator=(ThisType&& moveFrom)
	{
		Clear();

		Handle = moveFrom.Handle;
		moveFrom.Handle = nullptr;

		return *this;
	}

	bool operator==(const ThisType& Rhs) const
	{
		return (GDKHandleTraits::CompareHandle<HandleType>(*this, Rhs) == 0);
	}

	operator HandleType() const
	{
		return Handle;
	}

	operator const HandleType*() const
	{
		return IsValid() ? &Handle : nullptr;
	}

	operator bool() const
	{
		return IsValid();
	}

	bool IsValid() const
	{
		return Handle != nullptr;
	}

	void Reset(HandleType handle)
	{
		Clear();

		Handle = handle;
	}

	void Clear()
	{
		if (Handle)
		{
			GDKHandleTraits::ReleaseHandle(Handle);
			Handle = nullptr;
		}
	}

	HandleType* GetInitReference()
	{
		checkf(Handle == nullptr,TEXT("GDK handle already initalized"));
		return &Handle;
	}

	HandleType* ReleaseAndGetInitReference()
	{
		Clear();
		return &Handle;
	}

	/** Needed for TMap::GetTypeHash() */
	friend SIZE_T GetTypeHash(const ThisType& A)
	{
		return reinterpret_cast<SIZE_T>(A.Handle);
	}

private:
	HandleType Handle;
};

#define SETHANDLETYPETRAITS(handleType, duplicateMethod, releaseMethod) \
	namespace GDKHandleTraits { \
		template<> void DuplicateHandle(handleType Source, handleType* Target) \
		{ duplicateMethod(Source, Target); } \
		template<> void ReleaseHandle(handleType Handle) \
		{ releaseMethod(Handle); } \
	}

#define SETHANDLETYPETRAITSEX(handleType, duplicateMethod, releaseMethod, compareMethod) \
	namespace GDKHandleTraits { \
		template<> void DuplicateHandle(handleType Source, handleType* Target) \
		{ duplicateMethod(Source, Target); } \
		template<> void ReleaseHandle(handleType Handle) \
		{ releaseMethod(Handle); } \
		template<> int32 CompareHandle(handleType Lhs, handleType Rhs) \
		{ return compareMethod(Lhs, Rhs); } \
	}

SETHANDLETYPETRAITS(XTaskQueueHandle, XTaskQueueDuplicateHandle, XTaskQueueCloseHandle);
SETHANDLETYPETRAITSEX(XUserHandle, XUserDuplicateHandle, XUserCloseHandle, XUserCompare);

typedef TGDKHandle<XTaskQueueHandle> FGDKTaskQueueHandle;
typedef TGDKHandle<XUserHandle> FGDKUserHandle;


#endif //WITH_GRDK
