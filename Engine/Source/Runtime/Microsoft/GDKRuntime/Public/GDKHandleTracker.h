// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#define UE_API GDKRUNTIME_API

/* Whether to enable XSystemHandleTrack support. 
 *
 * It can be activated in a number of ways:
 *     Use the   -gdkhandletrack   command-line parameter for global tracking (unclosed handles will be dumped on exit)
 *     Use GDK.XSystemHandle.TrackStart / GDK.XSystemHandle.TrackStop for on-demand tracking.
 *     Use GDK_SCOPE_TRACK_HANDLES() to track any handles created in the current scope (expected use case is for localized debugging only)
 * 
 * Note that only one can be active at any one time
 */
#ifndef WITH_GDK_HANDLE_TRACKER
	#define WITH_GDK_HANDLE_TRACKER (!UE_BUILD_SHIPPING && !ENABLE_PGO_PROFILE && WITH_GRDK)
#endif

#if WITH_GDK_HANDLE_TRACKER

	class FGDKHandleTracker
	{
	public:
		inline FGDKHandleTracker()   { Start(); }
		inline ~FGDKHandleTracker()  { Stop(); }

		static UE_API void Start();
		static UE_API void Stop();
	};

	// track GDK handles that were created while the given scope was active. (note: this tracks all GDK handles in all threads.)
	// this is expected to be used for very localized debugging
	#define GDK_SCOPE_TRACK_HANDLES() FGDKHandleTracker ANONYMOUS_VARIABLE( GDKHandleTrack )

#else

	// no-op
	#define GDK_SCOPE_TRACK_HANDLES()


#endif //WITH_GDK_HANDLE_TRACKER

#undef UE_API
