// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if !defined(WITH_GRDK)
	#define WITH_GRDK 1
#endif

#if WITH_GRDK

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "Templates/Function.h"
#include "Delegates/Delegate.h"
#include "Misc/Optional.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "GDKHandle.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <grdk.h>
#include <XAsync.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#ifndef WITH_GRDK_DEV_RUNTIME_INIT
	#define WITH_GRDK_DEV_RUNTIME_INIT PLATFORM_WINDOWS && !UE_BUILD_SHIPPING && (_GRDK_EDITION >= 240600)
#endif

enum class XUserGamertagComponent: uint32_t;
enum class XNetworkingConnectivityLevelHint : uint32_t;

GDKRUNTIME_API DECLARE_LOG_CATEGORY_EXTERN(LogGDK, Log, All);

using FGDKRuntimeErrorProcessDelegate = TTSDelegate<bool(const char* /*Msg*/)>;
using FGDKPickUserCompleteDelegate = TTSDelegate<void(HRESULT, FGDKUserHandle)>;

DECLARE_MULTICAST_DELEGATE_OneParam( FGDKOnGameInviteReceived, const FString&/*Uri*/ );
typedef FGDKOnGameInviteReceived::FDelegate FGDKOnGameInviteReceivedDelegate;

struct XblUserProfile;

/**
 * Setup GDK runtime if there is any code rely on it before loading IGDKRuntimeModule
 */
GDKRUNTIME_API bool SetupGDKEnvironment( const TCHAR* ConfigIniSection = nullptr );

/**
 * Shutdown GDK runtime, symmetrical call to SetupGDKEnvironment
 */
GDKRUNTIME_API void TeardownGDKEnvironment();

class IGDKRuntimeModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to the GDK runtime module instance
	 * @return Returns IGDKRuntimeModule singleton instance, loading the module on demand if needed
	 */
	static inline IGDKRuntimeModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IGDKRuntimeModule>("GDKRuntime");
	}

	/**
	 * Singleton-like access to the GDK runtime module instance
	 * @return Returns IGDKRuntimeModule singleton instance if the module is already loaded
	 */
	static inline IGDKRuntimeModule* TryGet()
	{
		return FModuleManager::GetModulePtr<IGDKRuntimeModule>("GDKRuntime");
	}

	/**
	 * Returns whether the GDK runtime was initialized successfully.
	 */
	virtual bool IsAvailable() const = 0;


	/**
	 * Returns the Title Id for the game. See the GDK documentation for details.
	 */
	virtual uint32 GetTitleId() const = 0;

	/**
	 * Returns the current XBL sandbox for the platform. See the GDK documentation for details.
	 */
	virtual FString GetXboxSandboxId() const = 0;

	/**
	 * Returns the SCID for the title. See the GDK documentation for details.
	 * If bAllowPlaceholder is true and no SCID has been defined, a placeholder one will be returned rather than an empty string.
	 */
	virtual FString GetPrimaryServiceConfigId( bool bAllowPlaceholder = true ) const = 0;

	/**
	 * Returns the gamertag for the given user handle
	 * @param UserHandle the user to query
	 * @return the gamertag for the user, or an empty string if the user is not valid
	 */
	virtual FString GetGamertag(const FGDKUserHandle& UserHandle) const = 0;

	/**
	 * Returns the gamertag for the given user handle using the given gamertag convension
	 * @param UserHandle        the user to query
	 * @param GamertagComponent the convention to use - Classic/Modern etc.
	 * @return the gamertag for the user, or an empty string if the user is not valid
	 */
	virtual FString GetGamertag(const FGDKUserHandle& UserHandle, XUserGamertagComponent GamertagComponent) const = 0;

	/**
	 * Returns the preferred default gametag component
	 */
	virtual XUserGamertagComponent GetDefaultGamertagComponent() const = 0;

	/**
	 * Returns the config section that the GDK configuration is stored in
	 * @return [Config Section] name
	 */
	virtual const TCHAR* GetConfigSectionName() const = 0;

	/**
	 * Returns whether the project was configured to use the simplified user model
	 */
	virtual bool IsUsingSimplifiedUserModel() const = 0;

	/**
	 * Returns whether networking has been initialized yet (see XNetworkingConnectivityHint in the GDK documentation for details.)
	 */
	virtual bool IsNetworkInitialized() const = 0;


	/** Simple wrapper for async GDK task execution. 
	 * @param InitFunction   code that will be executed on the game thread to start the operation (e.g. XUserAddAsync). Expected to return the result from the GDK async function.
	 * @param ResultFunction optional code that will be executed when the async operation has finished (e.g. XUserAddResult). Execution thread depends on TaskQueue.
	 * @param TaskQueue      optional task queue. Default is GetGenericTaskQueue which executes on a background thread and ResultFunction happens on the game thread.
	 * @return the result from InitFunction, so S_OK means the async task has started successfully and ResultFunction will be called
	*/
	virtual HRESULT AsyncGDKTask( TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction = nullptr, XTaskQueueHandle TaskQueue = nullptr ) const = 0;

	/** Simple wrapper for cancellable async GDK task execution. 
	 * @param OutMonitor     variable that will receive the task monitor, allowing cancellation of the task (ResultFunction will still be called, with E_ABORT)
	 * @param InitFunction   code that will be executed on the game thread to start the operation (e.g. XUserAddAsync). Expected to return the result from the GDK async function.
	 * @param ResultFunction optional code that will be executed when the async operation has finished (e.g. XUserAddResult). Execution thread depends on TaskQueue.
	 * @param TaskQueue      optional task queue. Default is GetGenericTaskQueue which executes on a background thread and ResultFunction happens on the game thread.
	 * @return the result from InitFunction, so S_OK means the async task has started successfully and ResultFunction will be called
	*/
	virtual HRESULT AsyncGDKTask( class FGDKAsyncTaskMonitor& OutMonitor, TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction = nullptr, XTaskQueueHandle TaskQueue = nullptr ) const = 0;


	/**
	 * Get a reusable task queue. Tasks execute on a background thread and completion callbacks happen on the game thread
	 */
	virtual XTaskQueueHandle GetGenericTaskQueue() const = 0;

	/**
	 * Get a reusable task queue. Tasks execute on a background thread and completion callbacks also happen on a background thread
	 */
	virtual XTaskQueueHandle GetBackgroundTaskQueue() const = 0;


	/**
	 * Returns the user handle for the given platform user id
	 * @param PlatformUserId the user to query
	 * @return user handle for the given platform user
	 */
	virtual FGDKUserHandle GetUserHandleByPlatformId(FPlatformUserId PlatformUserId) const = 0;

	/**
	 * Returns the user handle for the given XUID
	 * @param XUserId XUID to query
	 * @return user handle for the given Xbox User Id
	 */
	virtual FGDKUserHandle GetUserHandleByXUserId(uint64 XUserId) const = 0;

	/**
	 * Returns the platform user id for the given user handle
	 * @param UserHandle the user to query
	 * @return platform user id for the given user
	 */
	virtual FPlatformUserId GetPlatformIdByUserHandle(const FGDKUserHandle& UserHandle) = 0;

	/**
	 * Returns the XUID for the given platform user
	 * @param PlatformUserId the user to query
	 * @return Xbox User Id for the given platform user
	 */
	virtual TOptional<uint64> GetXUserIdByPlatformId(FPlatformUserId PlatformUserId) const = 0;

	/**
	 * Returns all current user handles
	 */
	virtual TArray<FGDKUserHandle> GetAllUserHandles() const = 0;

	/**
	 * Returns whether the given local user was previously signed in but is not presently signed in
	 * @param UserLocalId the user to query
	 * @return true if the user had previously signed in but has now signed out. Will return false when the user has completed signing back in
	 * @note this function uses XUserLocalId because the user handle would be invalid once the user is signed out
	 */
	virtual bool WasUserRecentlySignedOut(XUserLocalId UserLocalId) const = 0;

	/**
	 * The async call to pick a user through the system UI
	 * @param Options The options when show the user pick UI
	 * @param Delegate The delegate to receive the HRESULT of the call
	 */
	virtual void PickUserAsync(XUserAddOptions Options, FGDKPickUserCompleteDelegate Delegate) const = 0;

	/**
	 * Returns the platform user id that will be used to represent any gamepads that are not paired to a user
	 */
	virtual FPlatformUserId GetUnpairedPlatformId() const = 0;

	/**
	 * Set runtime error process delegate for specific gdk runtime error
	 */
	virtual void SetRuntimeErrorProcessDelegate(HRESULT hResult, FGDKRuntimeErrorProcessDelegate Delegate) = 0;

	/**
	 * Clear runtime error process delegate for specific gdk runtime error
	 */
	virtual void ClearRuntimeErrorProcessDelegate(HRESULT hResult) = 0;

	/**
	 * Register a callback for when the game receives an online game invitation. callback parameter is the matchmaking Uri
	 */
	virtual FDelegateHandle RegisterGameInviteReceivedDelegate( FGDKOnGameInviteReceivedDelegate Delegate ) = 0;

	/**
	 * Unregister a previously-registered game invite callback
	 */
	virtual void UnregisterGameInviteReceivedDelegate( const FDelegateHandle& DelegateHandle ) = 0;


	// ... FPlatformMisc support functions ...

	/**
	 * Show a message box if possible, otherwise print a message and return the default
	 * @param MsgType What sort of options are provided
	 * @param Text Specific message
	 * @param Caption String indicating the title of the message box
	 * @return Very strange convention...not really EAppReturnType, see implementation
	 */
	virtual EAppReturnType::Type MessageBoxExt(EAppMsgType::Type MsgType, const TCHAR* Text, const TCHAR* Caption) const = 0;

	/**
	 * Returns the current network connection type
	 */
	virtual ENetworkConnectionType GetNetworkConnectionType() const = 0;

	/**
	 * Returns whether WiFi connection is currently active
	 */
	virtual bool HasActiveWiFiConnection() const = 0;

	/**
	 * Prevents the screen from dimming, locking or powering down
	 */
	virtual void DisableScreenTimeout() = 0;

	/**
	 * Allows the screen to dimming, lock or powering down again
	 */
	virtual void EnableScreenTimeout() = 0;



	// ... internal functions ...

	virtual void Internal_HandlePlatformTextFieldVisible( bool bIsVisible ) = 0;
	virtual bool Internal_IsPlatformTextFieldVisible() const = 0;

	// ... PIE support ...
#if WITH_EDITOR
	virtual void Internal_InitForPIE() = 0;
	virtual void Internal_TeardownForPIE() = 0;
	virtual TMulticastDelegate<void()>& GetOnInitForPIE() = 0;
	virtual TMulticastDelegate<void()>& GetOnTeardownForPIE() = 0;
#endif
};



GDKRUNTIME_API const TCHAR* LexToString(XNetworkingConnectivityLevelHint Value);
GDKRUNTIME_API FString LexToString(const FGDKUserHandle& UserHandle);
GDKRUNTIME_API FString LexToString(const XUserLocalId& UserId);

#endif //WITH_GRDK
