// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GDKRuntimeModule.h"
#if WITH_GRDK
#include "CoreTypes.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Tasks/Task.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XTaskQueue.h>
#include <XAsync.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#define UE_API GDKRUNTIME_API

/*	Wrapper for XAsyncBlock that allows capture variables in the lambda 

	If lifetime management is not required, using AsyncGDKTask() is recommended instead
*/
typedef TFunction<void(class FGDKAsyncBlock*)> FGDKAsyncBlockDelegate;

class FGDKAsyncBlock
{
public:
	UE_API FGDKAsyncBlock(void* UserData, FGDKAsyncBlockDelegate Delegate, XTaskQueueHandle TaskQueue = nullptr);

	UE_API ~FGDKAsyncBlock();

	UE_API HRESULT GetStatus();

	UE_API XAsyncBlock* GetInnerBlockForGDKAPI();
	UE_API void* GetUserData();
	operator XAsyncBlock*() { return GetInnerBlockForGDKAPI(); }

protected:
	XAsyncBlock AsyncBlock;
	void* UserData;
	FGDKAsyncBlockDelegate Delegate;

	UE_API void DelegateWrapper();
};



/* 
	Wrapper for XTaskQueueHandle that manages clean shutdown automatically

	This can be used when a separate, custom task queue is required. In most cases FGDKAsyncTaskQueue::GetGenericQueue() should be sufficient
*/
class FGDKAsyncTaskQueue
{
public:
	UE_API FGDKAsyncTaskQueue( XTaskQueueDispatchMode WorkDispatchMode = XTaskQueueDispatchMode::ThreadPool, XTaskQueueDispatchMode CompletionDispatchMode = XTaskQueueDispatchMode::ThreadPool );
	UE_API ~FGDKAsyncTaskQueue();

	FORCEINLINE XTaskQueueHandle GetQueue() const { return Queue; }
	UE_API HRESULT BlockUntilComplete( XAsyncBlock& AsyncBlock ) const;
	UE_API void CancelPendingTasksAndDestroyQueue();


	// access to common task queues
	static UE_API XTaskQueueHandle GetGenericQueue();			// completion callbacks happen on the game thread
	static UE_API XTaskQueueHandle GetGenericSerialQueue();		// completion callbacks happen on the game thread and are guaranteed to execute before the next work task starts in a bg thread, ensuring Work -> Callback order
	static UE_API XTaskQueueHandle GetBackgroundTaskQueue();	// completion callbacks happen on a background thread

	// internal use - startup & shutdown the queue system
	static UE_API void PlatformInit();
	static UE_API void PlatformTeardown();

protected:
	XTaskQueueDispatchMode WorkDispatchMode;
	XTaskQueueDispatchMode CompletionDispatchMode;
	XTaskQueueHandle Queue;
};



/* Helper class for executing simple GDK async tasks locally. This is expected to be created on the stack. 
   (Use LocalGDKTask() for an easier to use equivalent.)

	FGDKLocalTaskBlock Block;
	if (SUCCEEDED( XUserAddAsync( XUserAddOptions::None, Block ) ) )
	{
		AsyncBlock.BlockUntilComplete();

		FGDKUserHandle User;
		XUserAddResult(Block, User.GetInitReference());
	}
*/
class FGDKLocalTaskBlock
{
public:
	UE_API FGDKLocalTaskBlock();
	UE_API ~FGDKLocalTaskBlock();

	operator XAsyncBlock*() { return &AsyncBlock; }

	UE_API HRESULT BlockUntilComplete();
	UE_API HRESULT GetStatus();

protected:
	XAsyncBlock AsyncBlock;
	FGDKAsyncTaskQueue Queue;
};



/* Simple wrapper for async GDK task execution. Returns the result from InitFunction, so S_OK means the async task has started successfully

    AsyncGDKTask( 
		[]( XAsyncBlock* Block )
		{
			return XUserAddAsync( XUserAddOptions::None, Block );
		},
		[]( XAsyncBlock* Block )
		{
			FGDKUserHandle User;
			XUserAddResult(Block, User.GetInitReference());
		}
	);
     
*/
UE_API HRESULT AsyncGDKTask( TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction = nullptr, XTaskQueueHandle TaskQueue = nullptr );




/* Simple wrapper for async GDK task execution that can be monitored via a helper class. Returns the result from InitFunction, so S_OK means the async task has started successfully

	FGDKAsyncTaskMonitor MyTask;
    AsyncGDKTask( 
		MyTask,
		[]( XAsyncBlock* Block )
		{
			return XUserAddAsync( XUserAddOptions::None, Block );
		},
		[]( XAsyncBlock* Block )
		{
			FGDKUserHandle User;
			XUserAddResult(Block, User.GetInitReference());
		}
	);
	MyTask.TryCancel(true);

*/
UE_API HRESULT AsyncGDKTask( class FGDKAsyncTaskMonitor& OutAsyncTaskMonitor, TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<void(XAsyncBlock*)> ResultFunction = nullptr, XTaskQueueHandle TaskQueue = nullptr );




/* Simple wrapper for local GDK task execution. Blocks until the task is completed. Returns the result from InitFunction or ResultFunction

    LocalGDKTask( 
		[]( XAsyncBlock* Block )
		{
			return XUserAddAsync( XUserAddOptions::None, Block );
		},
		[]( XAsyncBlock* Block )
		{
			FGDKUserHandle User;
			return XUserAddResult(Block, User.GetInitReference());
		}
	);
     
*/
UE_API HRESULT LocalGDKTask( TFunction<HRESULT(XAsyncBlock*)> InitFunction, TFunction<HRESULT(XAsyncBlock*)> ResultFunction = nullptr );


/* Simple UE::Task wrapper for GDK task execution. Task result contains the result from InitFunction or ResultFunction

 FGDKTask Task = LaunchGDKTask( 
		UE_SOURCE_LOCATION,
		[]( XAsyncBlock* Block )
		{
			return XUserAddAsync( XUserAddOptions::None, Block );
		},
		[]( XAsyncBlock* Block )
		{
			FGDKUserHandle User;
			return XUserAddResult(Block, User.GetInitReference());
		}
	);
*/
typedef UE::Tasks::TTask<HRESULT> FGDKTask;

// launches a gdk task for asynchronous execution
// returns a trivially relocatable instance that can be used to wait for task completion or to obtain task execution result
UE_API FGDKTask LaunchGDKTask( 
	const TCHAR* DebugName, 
	TUniqueFunction<HRESULT(XAsyncBlock*)> InitFunction, 
	TUniqueFunction<HRESULT(XAsyncBlock*)> ResultFunction, 
	UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal );

UE_API FGDKTask LaunchGDKTask( 
	const TCHAR* DebugName, 
	class FGDKAsyncTaskMonitor& OutAsyncTaskMonitor,
	TUniqueFunction<HRESULT(XAsyncBlock*)> InitFunction, 
	TUniqueFunction<HRESULT(XAsyncBlock*)> ResultFunction, 
	UE::Tasks::ETaskPriority Priority = UE::Tasks::ETaskPriority::Normal );







/* Helper allowing cancellation from AsyncGDKTask() and LaunchGDKTask() */
class FGDKAsyncTaskMonitor
{
public:
	// Attempt to cancel the task. Note that the ResultFunction callback will be called and the GDK API function will likely return E_ABORT
	//  - When bWait is false, returns true if cancel request was made. 
	//  - When bWait is true it returns true if the task was successfully cancelled. This will happen before the ResultFunction is called
	//  - Returns false in all other failure cases, including if the task has already completed.
	UE_API bool TryCancel(bool bWait = true);

	// Determines if the task that we're monitoring is still valid. This will be true until the ResultFunction callback has completed
	UE_API bool IsValid() const;

protected:
	friend void Internal_InitAsyncTaskMonitor( FGDKAsyncTaskMonitor& TaskMonitor, TSharedPtr<FGDKAsyncBlock> Block );
	TWeakPtr<FGDKAsyncBlock> WeakBlock;
};


#undef UE_API

#endif //WITH_GRDK
