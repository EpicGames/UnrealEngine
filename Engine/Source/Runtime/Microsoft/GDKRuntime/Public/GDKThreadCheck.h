// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

// whether to enable XThreadSetTimeSensitive for RHI, Render and Game threads (depending on WITH_GDK_THREAD_CHECK_ALWAYS or -TimeSensitiveThreadCheck too)
#ifndef WITH_GDK_THREAD_CHECK
	#define WITH_GDK_THREAD_CHECK (!UE_BUILD_SHIPPING && !UE_BUILD_TEST && !ENABLE_PGO_PROFILE && WITH_GRDK)
#endif

// when set to 1 the -TimeSensitiveThreadCheck command line parameter is not needed
#ifndef WITH_GDK_THREAD_CHECK_ALWAYS
	#define WITH_GDK_THREAD_CHECK_ALWAYS 0
#endif


#if WITH_GDK_THREAD_CHECK

	class FGDKScopedNotTimeSensitive
	{
	public:
#if IS_MONOLITHIC
		FGDKScopedNotTimeSensitive();
		GDKRUNTIME_API ~FGDKScopedNotTimeSensitive();
#else
		inline FGDKScopedNotTimeSensitive();
		inline ~FGDKScopedNotTimeSensitive();
#endif //IS_MONOLITHIC

	private:
		bool bWasTimeSensitive;
	};

	// temporarily disables the time sensitive thread check in the current scope
	#define GDK_SCOPE_NOT_TIME_SENSITIVE() FGDKScopedNotTimeSensitive ANONYMOUS_VARIABLE( GDKNotTimeSenstitive )


	// runtime check if time sensitive thread checking should be enabled
	extern bool GDKRUNTIME_API IsGDKTimeSensitiveThreadCheckEnabled();

#else

	// no-op
	#define GDK_SCOPE_NOT_TIME_SENSITIVE()

#endif //WITH_GDK_THREAD_CHECK




// NOTE: modular game builds are not a supported feature
// this is here just to make sure it can still compile
#if WITH_GDK_THREAD_CHECK && !IS_MONOLITHIC

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <XThread.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"

inline FGDKScopedNotTimeSensitive::FGDKScopedNotTimeSensitive()
{
	bWasTimeSensitive = XThreadIsTimeSensitive();
	XThreadSetTimeSensitive(false);
}

inline FGDKScopedNotTimeSensitive::~FGDKScopedNotTimeSensitive()
{
	XThreadSetTimeSensitive(bWasTimeSensitive);
}
#endif //IS_MONOLITHIC

