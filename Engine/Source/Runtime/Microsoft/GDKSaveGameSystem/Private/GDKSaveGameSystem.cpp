// Copyright Epic Games, Inc. All Rights Reserved.

#include "GDKSaveGameSystem.h"
#if WITH_GRDK
#include "GDKRuntimeModule.h"
#include "GDKTaskQueueHelpers.h"
#include "GDKThreadCheck.h"
#include "GameDelegates.h"
#include "Async/Async.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/DateTime.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformMisc.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/ThreadHeartBeat.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <XThread.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"


DEFINE_LOG_CATEGORY(LogGDKSaveGame);

static int32 GGDKUseXDKCompatibleSave = 0;
static FAutoConsoleVariableRef CVarGDKUseXDKCompatibleSave(
	TEXT("GDK.UseXDKCompatibleSave"),
	GGDKUseXDKCompatibleSave,
	TEXT("Whether the savegame system will read and write XDK save games."),
	ECVF_Default
);

static int32 GGDKSaveGameSyncOnDemand = 0;
static FAutoConsoleVariableRef CVarGDKSaveGameSyncOnDemand(
	TEXT("GDK.SaveGame.UseSyncOnDemand"),
	GGDKSaveGameSyncOnDemand,
	TEXT("Sync saves on demand. Only necessary for games with a lot of very large containers. See the GDK documentation for details."),
	ECVF_Default
);



const uint32 FGDKSaveGameSystem::MaxContainerName = 256;
const char* FGDKSaveGameSystem::DefaultBlobName = "Data";

const char* FGDKSaveGameSystem::XDKCompatibleContainerName = "UE4DefaultSaveGameContainer";


FGDKSaveGameSystem::FGDKSaveGameSystem()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) XUserRegisterForChangeEvent is not safe to call on a time-sensitive thread

	//register for a callback when a user signs out
	auto UserChangeFn = [](void* Context, XUserLocalId UserLocalId, XUserChangeEvent Event)
	{
		((FGDKSaveGameSystem*)Context)->OnUserChange(UserLocalId, Event);
	};
	XUserRegisterForChangeEvent(FGDKAsyncTaskQueue::GetGenericQueue(), this, UserChangeFn, &UserChangeCallbackToken);

}

FGDKSaveGameSystem::~FGDKSaveGameSystem()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) XUserUnregisterForChangeEvent & XGameSaveCloseProvider are not safe to call on a time-sensitive thread

	XUserUnregisterForChangeEvent(UserChangeCallbackToken, true);

	for (auto Itr : PerUserSaveProviders)
	{
		XGameSaveCloseProvider(Itr.Value);
	}
	PerUserSaveProviders.Reset();
}

void FGDKSaveGameSystem::OnUserChange(XUserLocalId UserLocalId, XUserChangeEvent Event)
{
	// if the user signs out, remove them from the map - otherwise we risk E_GS_HANDLE_EXPIRED
	if (Event == XUserChangeEvent::SignedOut)
	{
		GDK_SCOPE_NOT_TIME_SENSITIVE(); // XUserFindUserByLocalId and XGameSaveCloseProvider are not safe to call on a time-sensitive thread
	
		FGDKUserHandle UserHandle;
		XUserFindUserByLocalId(UserLocalId, UserHandle.GetInitReference());

		XGameSaveProviderHandle Provider = nullptr;
		if (PerUserSaveProviders.RemoveAndCopyValue(UserHandle, Provider))
		{
			XGameSaveCloseProvider(Provider);
		}
	}
}


UE::Tasks::FTask FGDKSaveGameSystem::InitSaveGameProviderAsync(FPlatformUserId PlatformUserId)
{
	return AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, PlatformUserId]
		{
			const FGDKUserHandle User = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(PlatformUserId);
			if (!User.IsValid())
			{
				UE_LOGF(LogGDKSaveGame, Warning, "User %d is not valid", PlatformUserId.GetInternalId());
				return;
			}

			XGameSaveProviderHandle Provider = PerUserSaveProviders.FindRef(User);
			if (Provider == nullptr)
			{
				FString PrimaryServiceConfigId = IGDKRuntimeModule::Get().GetPrimaryServiceConfigId(false);
				if (PrimaryServiceConfigId.IsEmpty())
				{
					UE_LOGF(LogGDKSaveGame, Warning, "PrimaryServiceConfigId is not set - please make sure Project Settings are configured correctly");
					return;
				}

				// add a nested task to initialize the provider. this may display on-screen UI items
				// our task will not complete until the nested task has completed.
				AddNested( LaunchGDKTask( UE_SOURCE_LOCATION,
					[this, User, PrimaryServiceConfigId](XAsyncBlock* Block)
					{
						bool bSyncOnDemand = (GGDKSaveGameSyncOnDemand != 0);

						UE_LOGF(LogGDKSaveGame, Log, "Initializing save provider for user %ls...", *LexToString(User));
						HRESULT hResult = XGameSaveInitializeProviderAsync(User, TCHAR_TO_UTF8(*PrimaryServiceConfigId), bSyncOnDemand, Block);
						ProcessErrorCode(hResult, TEXT("InitializeProviderInit"));

						return hResult;
					},
					[this, User](XAsyncBlock* Block)
					{
						XGameSaveProviderHandle NewProvider;
						HRESULT hResult = XGameSaveInitializeProviderResult(Block, &NewProvider);
						ProcessErrorCode(hResult, TEXT("InitializeProvider"));

						// if the user was asked to resolve a save (i.e. choose between one offline and a newer online one for example), and selected "Cancel" or "Choose Next Time"
						// hResult will be E_GS_USER_CANCELED *and* we'll have a valid Provider representing the old/offline one. It's not possible to close the Provider and return null
						// because otherwise the next time this is called we will get S_OK and the same Provider, with no resolve UI displayed until the application is restarted.
						PerUserSaveProviders.Add(User, NewProvider);

						return hResult;
					}
				));
			}
		}
	);
}

void FGDKSaveGameSystem::LoadGameIfExistsAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback)
{
	// InternalLoadGameAsync already does everything that InternalDoesSaveGameExistAsync does but uses custom queuing, so just skip the first call
	LoadGameAsync(bAttemptToUseUI, Name, PlatformUserId, Callback);
}

XGameSaveProviderHandle FGDKSaveGameSystem::GetProviderForUserId(FPlatformUserId PlatformUserId)
{
	if (!IGDKRuntimeModule::Get().IsAvailable())
	{
		return nullptr;
	}

	const FGDKUserHandle User = IGDKRuntimeModule::Get().GetUserHandleByPlatformId(PlatformUserId);
	if (!User.IsValid())
	{
		return nullptr;
	}

	XGameSaveProviderHandle* HandlePtr = PerUserSaveProviders.Find(User);
	if (HandlePtr == nullptr)
	{
		return nullptr;
	}

	return (*HandlePtr);
}


UE::Tasks::FTask FGDKSaveGameSystem::InternalGetSaveGameNamesAsync(FPlatformUserId PlatformUserId, TSharedRef<TArray<FString>> FoundSaves, FSaveGameAsyncGetNamesCallback Callback, TSharedPtr<bool> OutResult)
{
	// start the enumeration operation on a background thread
	return AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, PlatformUserId, FoundSaves, Callback, OutResult]()
		{
			bool bSucceeded = false;

			// look up the provider first
			XGameSaveProviderHandle Provider = GetProviderForUserId(PlatformUserId);
			if (Provider != nullptr)
			{
				// enumerate the save game names
				HRESULT hResult = S_OK;
				if (GGDKUseXDKCompatibleSave == 0)
				{
					// GDK save game names are the container names themselves
					auto GatherGDKContainerInfo = [](const XGameSaveContainerInfo* ContainerInfo, void* Context)
					{
						if (ContainerInfo->name != nullptr && (FCStringAnsi::Strcmp(ContainerInfo->name, XDKCompatibleContainerName) != 0)) // ignore XDK compatible saves when we're not looking for them
						{
							TArray<FString>* FoundSavesPtr = (TArray<FString>*)Context;
							FoundSavesPtr->Add(FString(FUTF8ToTCHAR(ContainerInfo->name)));
						}
						return true;
					};

					hResult = XGameSaveEnumerateContainerInfo(Provider, &FoundSaves.Get(), GatherGDKContainerInfo);
					ProcessErrorCode(hResult, TEXT("EnumerateContainerInfo"));
				}
				else if (DoesContainerExist(Provider, FString(XDKCompatibleContainerName)))
				{
					// XDK-compatible save game names are the blob names within the specifically-named container
					auto GatherXDKBlobInfo = [](const XGameSaveBlobInfo* BlobInfo, void* Context)
					{
						if (BlobInfo->name != nullptr)
						{
							TArray<FString>* FoundSavesPtr = (TArray<FString>*)Context;
							FoundSavesPtr->Add(FString(FUTF8ToTCHAR(BlobInfo->name)));
						}
						return true;
					};

					// create the XDK-compatible container & enumerate all blobs
					XGameSaveContainerHandle Container = nullptr;
					hResult = XGameSaveCreateContainer(Provider, XDKCompatibleContainerName, &Container);
					ProcessErrorCode(hResult, TEXT("CreateContainer"));
					if (Container)
					{
						hResult = XGameSaveEnumerateBlobInfo(Container, &FoundSaves.Get(), GatherXDKBlobInfo);
						ProcessErrorCode(hResult, TEXT("EnumerateBlobInfo"));

						XGameSaveCloseContainer(Container);
					}

				}
				bSucceeded = SUCCEEDED(hResult);
			}

			// store the result, if requested
			if (OutResult.IsValid())
			{
				*OutResult = bSucceeded;
			}

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[PlatformUserId, bSucceeded, Callback, FoundSaves]()
					{
						Callback(PlatformUserId, bSucceeded, FoundSaves.Get());
					}
				);
			}
		},
		// prerequisite - init save provider
		InitSaveGameProviderAsync(PlatformUserId)
	);
}


UE::Tasks::FTask FGDKSaveGameSystem::InternalDoesSaveGameExistAsync(const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncExistsCallback Callback, TSharedPtr<ESaveExistsResult> OutResult)
{
	FString SlotName(Name);

	// start the save operation on a background thread.
	return AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,		
		[this, SlotName, PlatformUserId, Callback, OutResult]()
		{
			ESaveExistsResult Result = ESaveExistsResult::UnspecifiedError;

			// look up the provider first. failure to find the provider is a different failure reason to 'does not exist'
			XGameSaveProviderHandle Provider = GetProviderForUserId(PlatformUserId);
			if (Provider != nullptr)
			{
				FString ContainerName, BlobName;
				GetContainerAndBlobNames(ContainerName, BlobName, SlotName);
				UE_LOGF(LogGDKSaveGame, Display, "Looking for blob '%ls' in container '%ls'", *BlobName, *ContainerName);

				// enumerate the blobs in the container and see if the one we want exists
				XGameSaveContainerHandle Container = nullptr;
				HRESULT hResult = XGameSaveCreateContainer(Provider, TCHAR_TO_UTF8(*ContainerName), &Container);
				ProcessErrorCode(hResult, TEXT("CreateContainer"));

				Result = ISaveGameSystem::ESaveExistsResult::DoesNotExist;
				if (Container)
				{
					uint32 DataSize = 0;
					Result = GetBlobInfoByName(Container, BlobName, DataSize);
					XGameSaveCloseContainer(Container);
				}
			}

			// store the result, if requested
			if (OutResult.IsValid())
			{
				*OutResult = Result;
			}

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, Result, Callback]()
					{
						Callback(SlotName, PlatformUserId, Result);
					}
				);
			}
		}, 
		// prerequisite - init save provider
		InitSaveGameProviderAsync(PlatformUserId)
	);		
}


UE::Tasks::FTask FGDKSaveGameSystem::InternalSaveGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<const TArray<uint8>> Data, FSaveGameAsyncOpCompleteCallback Callback, TSharedPtr<bool> OutResult)
{
	FString SlotName(Name);

	// start the save operation on a background thread.
	return AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, SlotName, PlatformUserId, Data, Callback, OutResult]()
		{
			bool bSucceeded = false;

			// look up the provider first
			XGameSaveProviderHandle Provider = GetProviderForUserId(PlatformUserId);
			if (Provider != nullptr)
			{
				FString ContainerName, BlobName;
				GetContainerAndBlobNames(ContainerName, BlobName, SlotName);
				UE_LOGF(LogGDKSaveGame, Display, "Saving Blob '%ls' in Container '%ls'...", *BlobName, *ContainerName);

				XGameSaveContainerHandle Container = nullptr;
				XGameSaveUpdateHandle Update = nullptr;

				// create the container object (or open existing)
				HRESULT hResult = XGameSaveCreateContainer(Provider, TCHAR_TO_UTF8(*ContainerName), &Container);
				ProcessErrorCode(hResult, TEXT("CreateContainer"));

				if (SUCCEEDED(hResult))
				{
					hResult = XGameSaveCreateUpdate(Container, TCHAR_TO_UTF8(*ContainerName), &Update);
					ProcessErrorCode(hResult, TEXT("CreateUpdate"));
				}
				if (SUCCEEDED(hResult))
				{
					hResult = XGameSaveSubmitBlobWrite(Update, TCHAR_TO_UTF8(*BlobName), Data->GetData(), Data->Num());
					ProcessErrorCode(hResult, TEXT("SubmitBlobWrite"));
				}
				if (SUCCEEDED(hResult))
				{
					hResult = XGameSaveSubmitUpdate(Update);
					ProcessErrorCode(hResult, TEXT("SubmitUpdate"));
				}

				if (Update)
				{
					XGameSaveCloseUpdate(Update);
				}
				if (Container)
				{
					XGameSaveCloseContainer(Container);
				}

				bSucceeded = SUCCEEDED(hResult);
			}

			// store the result, if requested
			if (OutResult.IsValid())
			{
				*OutResult = bSucceeded;
			}

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, bSucceeded, Callback]()
					{
						Callback(SlotName, PlatformUserId, bSucceeded);
					}
				);
			}
		},
		// prerequisite - init save provider
		InitSaveGameProviderAsync(PlatformUserId)
	);
}


UE::Tasks::FTask FGDKSaveGameSystem::InternalLoadGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<TArray<uint8>> Data, FSaveGameAsyncLoadCompleteCallback Callback, TSharedPtr<bool> OutResult)
{
	FString SlotName(Name);

	// start the save operation on a background thread.
	return AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, SlotName, PlatformUserId, Data, Callback, OutResult]()
		{
			bool bSucceeded = false;

			// look up the provider first
			XGameSaveProviderHandle Provider = GetProviderForUserId(PlatformUserId);
			if (Provider != nullptr)
			{
				FString ContainerName, BlobName;
				GetContainerAndBlobNames(ContainerName, BlobName, SlotName);
				UE_LOGF(LogGDKSaveGame, Display, "Loading blob '%ls' from container '%ls'...", *BlobName, *ContainerName);

				if (DoesContainerExist(Provider, ContainerName))
				{
					// create the container object (or open existing)
					XGameSaveContainerHandle Container = nullptr;
					HRESULT hResult = XGameSaveCreateContainer(Provider, TCHAR_TO_UTF8(*ContainerName), &Container);
					ProcessErrorCode(hResult, TEXT("CreateContainer"));

					// read the blob
					uint32 DataSize = 0;
					if (Container != nullptr && GetBlobInfoByName(Container, BlobName, DataSize) == ISaveGameSystem::ESaveExistsResult::OK)
					{
						uint32 AllocationSize = DataSize;
						AllocationSize += BlobName.Len() + 1; // +1 for null
						AllocationSize += sizeof(XGameSaveBlob); // header

						FTCHARToUTF8 BlobNameUTF8(*BlobName);
						const char* BlobNames[] = { BlobNameUTF8.Get() };

						uint32 BlobCount = 1;
						XGameSaveBlob* Blob = (XGameSaveBlob*)FMemory::Malloc(AllocationSize);
						hResult = XGameSaveReadBlobData(Container, BlobNames, &BlobCount, AllocationSize, Blob);
						ProcessErrorCode(hResult, TEXT("ReadBlobData"));

						UE_CLOGF(BlobCount != 1, LogGDKSaveGame, Error, "Expected 1 blob, but got %d", BlobCount);
						if (SUCCEEDED(hResult) && BlobCount == 1)
						{
							int32 Offset = Data->AddUninitialized(Blob->info.size);
							FMemory::Memcpy(Data->GetData() + Offset, Blob->data, Blob->info.size);
							bSucceeded = true;
						}

						FMemory::Free(Blob);
					}

					if (Container)
					{
						XGameSaveCloseContainer(Container);
					}
				}
			}

			// store the result, if requested
			if (OutResult.IsValid())
			{
				*OutResult = bSucceeded;
			}

			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, bSucceeded, Callback, Data]()
					{
						Callback(SlotName, PlatformUserId, bSucceeded, Data.Get());
					}
				);
			}
		},
		// prerequisite - init save provider
		InitSaveGameProviderAsync(PlatformUserId)
	);
}


UE::Tasks::FTask FGDKSaveGameSystem::InternalDeleteGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncOpCompleteCallback Callback, TSharedPtr<bool> OutResult)
{
	FString SlotName(Name);

	// start the delete operation on a background thread.
	return AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, SlotName, PlatformUserId, Callback, OutResult]()
		{
			bool bSucceeded = false;

			// look up the provider first
			XGameSaveProviderHandle Provider = GetProviderForUserId(PlatformUserId);
			if (Provider != nullptr)
			{
				FString ContainerName, BlobName;
				GetContainerAndBlobNames(ContainerName, BlobName, SlotName);
				UE_LOGF(LogGDKSaveGame, Display, "Deleting blob '%ls' from container '%ls'...", *BlobName, *ContainerName);

				XGameSaveContainerHandle Container = nullptr;
				XGameSaveUpdateHandle Update = nullptr;

				if (DoesContainerExist(Provider, ContainerName))
				{
					// create the container object (or open existing)
					HRESULT hResult = XGameSaveCreateContainer(Provider, TCHAR_TO_UTF8(*ContainerName), &Container);
					ProcessErrorCode(hResult, TEXT("CreateContainer"));

					uint32 DataSize = 0;
					if (GetBlobInfoByName(Container, BlobName, DataSize) == ISaveGameSystem::ESaveExistsResult::OK)
					{
						if (SUCCEEDED(hResult))
						{
							hResult = XGameSaveCreateUpdate(Container, TCHAR_TO_UTF8(*ContainerName), &Update);
							ProcessErrorCode(hResult, TEXT("CreateUpdate"));
						}
						if (SUCCEEDED(hResult))
						{
							hResult = XGameSaveSubmitBlobDelete(Update, TCHAR_TO_UTF8(*BlobName));
							ProcessErrorCode(hResult, TEXT("SubmitBlobDelete"));
						}
						if (SUCCEEDED(hResult))
						{
							hResult = XGameSaveSubmitUpdate(Update);
							ProcessErrorCode(hResult, TEXT("SubmitUpdate"));
						}

						if (Update)
						{
							XGameSaveCloseUpdate(Update);
						}
					}

					uint32 NumBlobsRemaining = GetNumBlobsInContainer(Container);
					if (Container)
					{
						XGameSaveCloseContainer(Container);
					}

					bSucceeded = SUCCEEDED(hResult);

					// delete the container if it is now empty
					if (NumBlobsRemaining == 0)
					{
						UE_LOGF(LogGDKSaveGame, Display, "Deleting empty container '%ls'...", *ContainerName);
						hResult = XGameSaveDeleteContainer(Provider, TCHAR_TO_UTF8(*ContainerName));
						ProcessErrorCode(hResult, TEXT("DeleteContainer"));
					}
				}
			}


			// store the result, if requested
			if (OutResult.IsValid())
			{
				*OutResult = bSucceeded;
			}


			// trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[SlotName, PlatformUserId, bSucceeded, Callback]()
					{
						Callback(SlotName, PlatformUserId, bSucceeded);
					}
				);
			}
		},
		// prerequisite - init save provider
		InitSaveGameProviderAsync(PlatformUserId)
	);
}



void FGDKSaveGameSystem::InitAsync(bool bAttemptToUseUI, FPlatformUserId PlatformUserId, FSaveGameAsyncInitCompleteCallback Callback)
{
	// start the init operation on a background thread
	AsyncTaskPipe.Launch(UE_SOURCE_LOCATION,
		[this, PlatformUserId, Callback]
		{
			// get the save provider handle
			XGameSaveProviderHandle Provider = GetProviderForUserId(PlatformUserId);
			bool bSuccess = (Provider != nullptr);

			//trigger the callback on the game thread
			if (Callback)
			{
				OnAsyncComplete(
					[PlatformUserId, bSuccess, Callback]()
					{
						Callback(PlatformUserId, bSuccess);
					}
				);
			}
		},
		// prerequisite - init save provider
		InitSaveGameProviderAsync(PlatformUserId)
	);
}



void FGDKSaveGameSystem::WaitForAsyncTask(UE::Tasks::FTask AsyncSaveTask)
{
	// need to pump messages on the game thread
	if (IsInGameThread())
	{
		// Suspend the hang and hitch heartbeats, as this is a long running task.
		FSlowHeartBeatScope SuspendHeartBeat;
		FDisableHitchDetectorScope SuspendGameThreadHitch;

		while (!AsyncSaveTask.IsCompleted())
		{
			FPlatformMisc::PumpMessagesOutsideMainLoop();
		}
	}
	else
	{
		// not running on the game thread, so just block until the async operation comes back
		AsyncSaveTask.Wait();
	}
}



void FGDKSaveGameSystem::GetContainerAndBlobNames( FString& OutContainerName, FString& OutBlobName, const FString& InSaveName )
{
	if (GGDKUseXDKCompatibleSave)
	{
		OutContainerName = XDKCompatibleContainerName;
		OutBlobName = InSaveName;
	}
	else
	{
		OutContainerName = SanitizeContainerName(*InSaveName);
		OutBlobName = DefaultBlobName;
	}
}


FString FGDKSaveGameSystem::SanitizeContainerName(const TCHAR* Name)
{
	TCHAR ContainerName[MaxContainerName];
	int WriteIdx = 0;
	while( WriteIdx < (MaxContainerName-1) && *Name != '\0')
	{
		if (FChar::IsAlnum(*Name) || *Name == '_' || *Name == '-')
		{
			ContainerName[WriteIdx++] = *Name;
		}
		Name++;
	}
	ContainerName[WriteIdx++] = '\0';

	return (ContainerName[0] == '\0') ? TEXT("Default") : ContainerName;
}



ISaveGameSystem::ESaveExistsResult FGDKSaveGameSystem::GetBlobInfoByName( XGameSaveContainerHandle InContainer, const FString& InBlobName, uint32& OutDataSize )
{
	// helper for enumerating blobs
	struct FBlobEnumData
	{
		FBlobEnumData( const FString& InName, XGameSaveContainerHandle InContainer ) :
			Container(InContainer),
			Name(*InName),
			Size(0),
			bFound(false)
		{}

		XGameSaveContainerHandle Container;
		FTCHARToUTF8 Name;
		uint32 Size;
		bool bFound;

		void Enumerate()
		{
			HRESULT hResult = XGameSaveEnumerateBlobInfo( Container, this, []( const XGameSaveBlobInfo* Info, void* Context )
			{
				FBlobEnumData* BlobEnumData = (FBlobEnumData*)Context;
				if (FCStringAnsi::Strcmp(Info->name, BlobEnumData->Name.Get()) == 0) //NB. blob names are case-sensitive
				{
					BlobEnumData->Size = Info->size;
					BlobEnumData->bFound = true;
					return false;
				}
				return true;
			});

			FGDKSaveGameSystem::ProcessErrorCode(hResult, TEXT("EnumerateBlobInfo"));
		}
	};

	// enumerate the blobs and find the size of the one we are looking for
	FBlobEnumData BlobEnumData(InBlobName, InContainer);

	BlobEnumData.Enumerate();

	if (BlobEnumData.bFound)
	{
		UE_LOGF(LogGDKSaveGame, Display, "Found blob '%ls' - %d bytes", *InBlobName, BlobEnumData.Size );

		OutDataSize = BlobEnumData.Size;
		return ISaveGameSystem::ESaveExistsResult::OK;
	}
	else
	{
		UE_LOGF(LogGDKSaveGame, Display, "Blob '%ls' not found", *InBlobName );

		OutDataSize = 0;
		return ISaveGameSystem::ESaveExistsResult::DoesNotExist;
	}
}


uint32 FGDKSaveGameSystem::GetNumBlobsInContainer( XGameSaveContainerHandle InContainer )
{
	uint32 Result = 0;
	HRESULT hResult = XGameSaveEnumerateBlobInfo( InContainer, &Result, []( const XGameSaveBlobInfo* Info, void* Context )
	{
		(*static_cast<uint32*>(Context))++;
		return true;
	});
	ProcessErrorCode(hResult, TEXT("EnumerateBlobInfo"));

	return Result;
}


bool FGDKSaveGameSystem::DoesContainerExist( XGameSaveProviderHandle InProvider, const FString& InContainerName )
{
	bool bContainerExists = false;
	HRESULT hResult = XGameSaveGetContainerInfo( InProvider, TCHAR_TO_UTF8(*InContainerName), &bContainerExists, [](const XGameSaveContainerInfo* Container, void* Context) //NB. container names are case sensitive
	{
		(*static_cast<bool*>(Context)) = true;
		return true;
	});
	ProcessErrorCode(hResult, TEXT("ContainerInfo") );

	UE_CLOGF(!bContainerExists, LogGDKSaveGame, Log, "Container '%ls' not found", *InContainerName );
	return bContainerExists;
}


void FGDKSaveGameSystem::ProcessErrorCode( HRESULT Result, const TCHAR* Msg )
{
	switch(Result)
	{
		case S_OK:                       return;
		case E_GS_NO_ACCESS:             UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_NO_ACCESS: SCID, TitleID or package identity may not be set up correctly.", Msg); break;
		case E_GS_OUT_OF_LOCAL_STORAGE:  UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_OUT_OF_LOCAL_STORAGE.", Msg); break;
		case E_GS_UPDATE_TOO_BIG:        UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_UPDATE_TOO_BIG: reduce the savegame size", Msg); break;
		case E_GS_QUOTA_EXCEEDED:        UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_QUOTA_EXCEEDED: reduce the savegame size", Msg); break;
		case E_GS_CONTAINER_NOT_IN_SYNC: UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_CONTAINER_NOT_IN_SYNC", Msg); break;
		case E_GS_CONTAINER_SYNC_FAILED: UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_CONTAINER_SYNC_FAILED", Msg); break;
		case E_GS_HANDLE_EXPIRED:        UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_HANDLE_EXPIRED", Msg); break;
		case E_GS_USER_CANCELED:         UE_LOGF(LogGDKSaveGame, Error, "%ls: E_GS_USER_CANCELED", Msg); break;
		default:                         UE_LOGF(LogGDKSaveGame, Error, "%ls: Failed 0x%X", Msg, Result ); break;
	}
}

void FGDKSaveGameSystem::DebugListSavesForAllUsers()
{
	TArray<FGDKUserHandle> AllUserHandles = IGDKRuntimeModule::Get().GetAllUserHandles();
	for (FGDKUserHandle UserHandle : AllUserHandles )
	{
		const FPlatformUserId PlatformUserId = IGDKRuntimeModule::Get().GetPlatformIdByUserHandle(UserHandle);
		const int32 UserIndex = FPlatformMisc::GetUserIndexForPlatformUser(PlatformUserId);
		DebugListSavesForUser(UserIndex);
	}
}

void FGDKSaveGameSystem::DebugListSavesForUser( const int32 UserIndex )
{
	const FPlatformUserId PlatformUserId = FPlatformMisc::GetPlatformUserForUserIndex(UserIndex);
	XGameSaveProviderHandle Provider = GetProviderForUserId(PlatformUserId);
	if (Provider == nullptr)
	{
		return;
	}

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (debug only) XGameSaveEnumerateContainerInfo & XGameSaveEnumerateBlobInfo are not safe to call on a time-sensitive thread

	auto DumpContainerInfo = []( const XGameSaveContainerInfo* Info, void* Context )
	{
		bool bIsXDKCompatibleContainer = Info->name && (FCStringAnsi::Strcmp( Info->name, XDKCompatibleContainerName ) == 0);
		FDateTime ModifiedTime = FDateTime::FromUnixTimestamp(Info->lastModifiedTime);
		FString ModifiedAgo = (FDateTime::UtcNow() - ModifiedTime).ToString(TEXT("%dd %hh %mmin %ss ago"));
		ModifiedAgo.RemoveFromStart(TEXT("+"));

		UE_LOGF(LogGDKSaveGame, Display, "\t%lsContainer: '%ls' %lld bytes (%ls)", bIsXDKCompatibleContainer ? TEXT("XDK-Compatible ") : TEXT(""), Info->name ? UTF8_TO_TCHAR(Info->name) : TEXT("<null>"), Info->totalSize, Info->needsSync ? TEXT("needs sync") : TEXT("synced"));
		if (Info->name != nullptr)
		{
			// create this container
			XGameSaveContainerHandle Container = nullptr;
			HRESULT hResult = XGameSaveCreateContainer( (XGameSaveProviderHandle)Context, Info->name, &Container );
			ProcessErrorCode( hResult, *FString::Printf( TEXT("cannot create save game container for %s"), UTF8_TO_TCHAR(Info->name) ) );

			// enumerate all blobs
			if (Container)
			{
				auto DumpBlobInfo = []( const XGameSaveBlobInfo* Info, void* Context )
				{
					UE_LOGF( LogGDKSaveGame, Display, "\t\tBlob: '%ls' %d bytes", Info->name ? UTF8_TO_TCHAR(Info->name) : TEXT("<null>"), Info->size );
					return true;
				};

				hResult = XGameSaveEnumerateBlobInfo( Container, nullptr, DumpBlobInfo );
				ProcessErrorCode( hResult, *FString::Printf( TEXT("cannot enumerate save games for container '%s'"), UTF8_TO_TCHAR(Info->name) ) );
				XGameSaveCloseContainer(Container);
			}
		}
		UE_LOGF(LogGDKSaveGame, Display, "\tLast modified: %ls  (%ls)", *ModifiedAgo, *ModifiedTime.ToString());
		UE_LOGF(LogGDKSaveGame, Display, "" );

		return true;
	};

	// enumerate all containers
	UE_LOGF(LogGDKSaveGame, Display, "*** User %d save data (%ls) ***", UserIndex, *LexToString(IGDKRuntimeModule::Get().GetUserHandleByPlatformId(PlatformUserId)) );
	UE_LOGF(LogGDKSaveGame, Display, "" );
	HRESULT hResult = XGameSaveEnumerateContainerInfo( Provider, (void*)Provider, DumpContainerInfo );
	ProcessErrorCode( hResult, TEXT("cannot enumerate save game containers") );
	UE_LOGF(LogGDKSaveGame, Display, "********" );
	UE_LOGF(LogGDKSaveGame, Display, "" );
}



#endif //WITH_GRDK
