// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "GDKRuntimeModule.h"
#include "GDKSaveGameSystem.h"

class FGDKSaveGameSystemModule : public ISaveGameSystemModule
{
public:
#if WITH_GRDK
	std::atomic<bool> bWantSystemCreation;
	TUniquePtr<FGDKSaveGameSystem> SaveGameSystem;

	FGDKSaveGameSystemModule()
		: bWantSystemCreation(true)
	{
#if WITH_EDITOR
		// tear down the savegame system when PIE finishes
		IGDKRuntimeModule::Get().GetOnTeardownForPIE().AddRaw( this, &FGDKSaveGameSystemModule::OnTeardownForPIE );
#else
		// prevent creation of GDK savegame system if the GDK runtime is unavailable
		if (!IGDKRuntimeModule::Get().IsAvailable())
		{
			bWantSystemCreation = false;
		}
#endif
	}

	virtual void ShutdownModule() override
	{
		SaveGameSystem.Reset();

#if WITH_EDITOR
		if (IGDKRuntimeModule* GDKRuntime = IGDKRuntimeModule::TryGet())
		{
			GDKRuntime->GetOnTeardownForPIE().RemoveAll(this);
		}
#endif
	}

	virtual ISaveGameSystem* GetSaveGameSystem() override
	{
#if WITH_EDITOR
		if (!IGDKRuntimeModule::Get().IsAvailable())
		{
			return nullptr;
		}
#endif

		if (bWantSystemCreation)
		{
			bWantSystemCreation = false;
			SaveGameSystem.Reset( new FGDKSaveGameSystem());
		}

		return SaveGameSystem.Get();
	}


	static inline FGDKSaveGameSystem* GetStatic()
	{
		FGDKSaveGameSystemModule& ThisModule = FModuleManager::LoadModuleChecked<FGDKSaveGameSystemModule>("GDKSaveGameSystem");
		return (FGDKSaveGameSystem*)ThisModule.GetSaveGameSystem();
	}

#if WITH_EDITOR
	void OnTeardownForPIE()
	{
		SaveGameSystem.Reset();
		bWantSystemCreation = true;
	}
#endif



#else

	// dummy
	virtual ISaveGameSystem* GetSaveGameSystem() override
	{
		return nullptr;
	}
#endif //WITH_GRDK
};


#if WITH_GRDK

static FAutoConsoleCommand CmdGDKListSaves(
	TEXT("GDK.SaveGame.List"),
	TEXT("[optional UserIndex] Print out users' save data"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		if (Args.Num() > 0)
		{
			int32 UserIndex = Args.Num() > 0 ? FCString::Atoi(*Args[0]) : 0;
			FGDKSaveGameSystemModule::GetStatic()->DebugListSavesForUser(UserIndex);
		}
		else
		{
			FGDKSaveGameSystemModule::GetStatic()->DebugListSavesForAllUsers();
		}
	})
);

static FAutoConsoleCommand CmdGDKDoesSaveExist(
	TEXT("GDK.SaveGame.Exists"),
	TEXT("[Name] [optional UserIndex] - Checks to see if a given save exists"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		if (Args.Num() > 0)
		{
			int32 UserIndex = Args.Num() > 1 ? FCString::Atoi(*Args[1]) : 0;
			bool bExists = FGDKSaveGameSystemModule::GetStatic()->DoesSaveGameExist(*Args[0], UserIndex );
			UE_LOGF( LogGDKSaveGame, Log, "Savegame %ls %ls", *Args[0], bExists ? TEXT("exists") : TEXT("does not exist") );
		}
	})
);

static FAutoConsoleCommand CmdGDKDeleteSave(
	TEXT("GDK.SaveGame.Delete"),
	TEXT("[Name] [optional UserIndex] - Deletes the given save"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		if (Args.Num() > 0)
		{
			int32 UserIndex = Args.Num() > 1 ? FCString::Atoi(*Args[1]) : 0;
			bool bSuccess = FGDKSaveGameSystemModule::GetStatic()->DeleteGame(false, *Args[0], UserIndex );
			UE_LOGF( LogGDKSaveGame, Log, "Savegame %ls was %ls", *Args[0], bSuccess ? TEXT("deleted") : TEXT("not deleted") );
			FGDKSaveGameSystemModule::GetStatic()->DebugListSavesForUser(UserIndex);
		}
	})
);

static FAutoConsoleCommand CmdGDKLoadBlob(
	TEXT("GDK.SaveGame.DummyLoad"),
	TEXT("[Name] [optional UserIndex] - Tries to load the given save data. note: deletes the data once loaded"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		if (Args.Num() > 0)
		{
			int32 UserIndex = Args.Num() > 1 ? FCString::Atoi(*Args[1]) : 0;
			TArray<uint8> Result;
			bool bSuccess = FGDKSaveGameSystemModule::GetStatic()->LoadGame(false, *Args[0], UserIndex, Result );
			UE_LOGF( LogGDKSaveGame, Log, "Savegame %ls was %ls", *Args[0], bSuccess ? TEXT("loaded") : TEXT("not loaded") );
			FGDKSaveGameSystemModule::GetStatic()->DebugListSavesForUser(UserIndex);
		}
	})
);

static FAutoConsoleCommand CmdGDKSaveBlob(
	TEXT("GDK.SaveGame.DummySave"),
	TEXT("[Name] [optional UserIndex] - Tries to save dummy data with the given name"),
	FConsoleCommandWithArgsDelegate::CreateLambda( [](const TArray<FString>& Args )
	{
		if (Args.Num() > 0)
		{
			int32 UserIndex = Args.Num() > 1 ? FCString::Atoi(*Args[1]) : 0;
			TArray<uint8> Data;
			Data.AddZeroed(256);
			bool bSuccess = FGDKSaveGameSystemModule::GetStatic()->SaveGame(false, *Args[0], UserIndex, Data );
			UE_LOGF( LogGDKSaveGame, Log, "Savegame %ls was %ls", *Args[0], bSuccess ? TEXT("saved") : TEXT("not saved") );
			FGDKSaveGameSystemModule::GetStatic()->DebugListSavesForUser(UserIndex);
		}
	})
);

static FAutoConsoleCommand CmdGDKInitSave(
	TEXT("GDK.SaveGame.Init"),
	TEXT("[optional UserIndex] initialise for this user"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		int32 UserIndex = Args.Num() > 0 ? FCString::Atoi(*Args[0]) : 0;
		FGDKSaveGameSystemModule::GetStatic()->InitAsync(false, FPlatformMisc::GetPlatformUserForUserIndex(UserIndex), [](FPlatformUserId id, bool succ)
		{
			UE_LOGF(LogGDKSaveGame, Log, "Savegame init %d %ls", id.GetInternalId(), succ ? TEXT("true") : TEXT("false"));
		});
	})
);

#endif //WITH_GRDK





IMPLEMENT_MODULE(FGDKSaveGameSystemModule, GDKSaveGameSystem);
