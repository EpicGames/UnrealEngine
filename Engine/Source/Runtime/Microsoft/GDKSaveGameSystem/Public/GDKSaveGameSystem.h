// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SaveGameSystem.h"
#if WITH_GRDK
#include "GDKHandle.h"

#include "Microsoft/AllowMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_START
#include <XGameSave.h>
#include <XTaskQueue.h>
THIRD_PARTY_INCLUDES_END
#include "Microsoft/HideMicrosoftPlatformTypes.h"

#define UE_API GDKSAVEGAMESYSTEM_API

DECLARE_LOG_CATEGORY_EXTERN(LogGDKSaveGame, Log, All);

class FGDKSaveGameSystem : public FBaseAsyncSaveGameSystem
{
public:
	UE_API FGDKSaveGameSystem();
	UE_API virtual ~FGDKSaveGameSystem();

	/** Returns true if the platform has a native UI (like many consoles) */
	virtual bool PlatformHasNativeUI() override
	{
		return false;
	}

	virtual bool DoesSaveSystemSupportMultipleUsers() override
	{
		return true;
	}

	/* (Optional) initialise the save system for the given user. GDK may display the 'syncing' or 'resolve save' dialog when creating the save provider */
	UE_API virtual void InitAsync(bool bAttemptToUseUI, FPlatformUserId PlatformUserId, FSaveGameAsyncInitCompleteCallback Callback) override;
	
	/** Override as the custom queuing breaks the default implementation and is unnecessary */
	UE_API virtual void LoadGameIfExistsAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncLoadCompleteCallback Callback) override;


	/** Debugging helper functions */
	UE_API void DebugListSavesForAllUsers();
	UE_API void DebugListSavesForUser( const int32 UserIndex );

private:
	static UE_API const uint32 MaxContainerName;
	static UE_API const char* DefaultBlobName;
	static UE_API const char* XDKCompatibleContainerName;

	// internal async helpers
	UE_API virtual UE::Tasks::FTask InternalDoesSaveGameExistAsync(const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncExistsCallback Callback, TSharedPtr<ESaveExistsResult> OutResult = nullptr) override;
	UE_API virtual UE::Tasks::FTask InternalSaveGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<const TArray<uint8>> Data, FSaveGameAsyncOpCompleteCallback Callback, TSharedPtr<bool> OutResult = nullptr) override;
	UE_API virtual UE::Tasks::FTask InternalLoadGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, TSharedRef<TArray<uint8>> Data, FSaveGameAsyncLoadCompleteCallback Callback, TSharedPtr<bool> OutResult = nullptr) override;
	UE_API virtual UE::Tasks::FTask InternalDeleteGameAsync(bool bAttemptToUseUI, const TCHAR* Name, FPlatformUserId PlatformUserId, FSaveGameAsyncOpCompleteCallback Callback, TSharedPtr<bool> OutResult = nullptr) override;
	UE_API virtual UE::Tasks::FTask InternalGetSaveGameNamesAsync(FPlatformUserId PlatformUserId, TSharedRef<TArray<FString>> FoundSaves, FSaveGameAsyncGetNamesCallback Callback, TSharedPtr<bool> OutResult = nullptr) override;

	UE_API UE::Tasks::FTask InitSaveGameProviderAsync(FPlatformUserId PlatformUserId);

	UE_API virtual void WaitForAsyncTask(UE::Tasks::FTask AsyncSaveTask) override;

	// user sign in/out callback
	XTaskQueueRegistrationToken UserChangeCallbackToken;
	UE_API void OnUserChange(XUserLocalId UserLocalId, XUserChangeEvent Event);

	// get the final blob and container names
	static UE_API void GetContainerAndBlobNames( FString& OutContainerName, FString& OutBlobName, const FString& InSaveName );

	// remove invalid characters from the given save container name
	static UE_API FString SanitizeContainerName(const TCHAR* Name);

	// get the info for a particular named blob
	static UE_API ESaveExistsResult GetBlobInfoByName( XGameSaveContainerHandle InContainer, const FString& InBlobName, uint32& OutDataSize );

	// get all blobs in the given container
	static UE_API uint32 GetNumBlobsInContainer( XGameSaveContainerHandle InContainer );

	// check whether the given container exists
	static UE_API bool DoesContainerExist( XGameSaveProviderHandle InProvider, const FString& InContainerName );
	
	// error display helper
	static UE_API void ProcessErrorCode( HRESULT Result, const TCHAR* Msg );

	// get a provider for the user's save provider
	UE_API XGameSaveProviderHandle GetProviderForUserId(FPlatformUserId UserId);

	// cache the save providers for each user
	TMap<FGDKUserHandle, XGameSaveProviderHandle> PerUserSaveProviders;
};

#undef UE_API

#endif //WITH_GRDK
